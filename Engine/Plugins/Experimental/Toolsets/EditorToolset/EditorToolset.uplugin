{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "EditorToolset",
	"Description": "Toolsets for interacting with the Unreal Editor and core types (Blueprints, Actors, Properties, etc.) via the AI Toolset Registry.",
	"Category": "Other",
	"EditorOnly": true,
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "EditorToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": [ "Editor" ]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}
