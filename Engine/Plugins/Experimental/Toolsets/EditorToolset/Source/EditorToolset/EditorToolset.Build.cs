// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class EditorToolset : ModuleRules
{
	public EditorToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"ContentBrowser",
				"ContentBrowserData",
				"CoreUObject",
				"EditorFramework",
				"EditorScriptingUtilities",
				"EditorSubsystem",
				"Engine",
				"Json",
				"JsonUtilities",
				"Kismet",
				"LevelEditor",
				"RenderCore",
				"RHI",
				"Slate",
				"SlateCore",
				"StatusBar",
				"ToolsetRegistry",
				"UnrealEd",
			}
			);
	}
}
