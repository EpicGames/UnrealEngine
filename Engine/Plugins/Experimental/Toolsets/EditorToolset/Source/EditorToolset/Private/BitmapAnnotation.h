// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/Matrix.h"
#include "Math/Vector.h"

namespace UE::ToolsetRegistry::BitmapAnnotation
{
	/**
	 * Lightweight view over an in-memory RGBA bitmap.
	 *
	 * Bundles the pixel buffer with its dimensions so drawing primitives can be
	 * called as member functions without the caller re-passing both every time.
	 * Holds a reference to the caller's pixel array — it does not own the memory
	 * and must not outlive the underlying TArray.
	 */
	struct FBitmapCanvas
	{
		TArray<FColor>& Pixels;
		FIntPoint Size;

		FBitmapCanvas(TArray<FColor>& InPixels, FIntPoint InSize)
			: Pixels(InPixels)
			, Size(InSize)
		{
		}

		// Pixel / blending primitives ----------------------------------------

		/** Write a single pixel, clipped to the bitmap bounds. */
		void SetPixel(int32 X, int32 Y, const FColor& Color);

		/** Alpha-blend a single pixel over the existing contents. */
		void BlendPixel(int32 X, int32 Y, const FColor& Color, float Alpha = 1.0f);

		// Line / dot primitives ----------------------------------------------

		/** Bresenham line with alpha blending and a runaway-iteration cap. */
		void DrawLine(int32 X0, int32 Y0, int32 X1, int32 Y1, const FColor& Color, float Alpha = 0.6f);

		/** Thick line drawn as parallel offset single-pixel lines. */
		void DrawLineThick(int32 X0, int32 Y0, int32 X1, int32 Y1, const FColor& Color, int32 Thickness = 2, float Alpha = 0.6f);

		/** Filled disc (grid intersection marker). */
		void DrawDot(int32 CX, int32 CY, int32 Radius, const FColor& Color);

		/** Draw a single character at (X, Y) with nearest-neighbour scaling. */
		void DrawChar(int32 X, int32 Y, TCHAR Ch, const FColor& Color, int32 Scale = 2);

		/** Draw a string at (X, Y). When bDrawBackground is true, a semi-transparent black plate is drawn behind the text for readability. */
		void DrawString(int32 X, int32 Y, const FString& Text, const FColor& Color, int32 Scale = 2, bool bDrawBackground = true);
	};

	// Tiny 5x7 bitmap font ---------------------------------------------------
	//
	// ASCII 32..95 ('_') map to glyphs. Lowercase a..z is folded to uppercase.
	// Unknown characters render as a space.

	int32 GetGlyphWidth();
	int32 GetGlyphHeight();

	/** Width in pixels of one character cell (glyph width + 1px inter-char spacing), at the given integer scale. */
	int32 GetGlyphAdvance(int32 Scale);

	/** Measured pixel width of a string at the given integer scale. */
	int32 MeasureString(const FString& Text, int32 Scale = 2);

	// High-level overlays ----------------------------------------------------

	/**
	 * One labeled point in an actor-label overlay pass.
	 *
	 * Caller does the actor iteration / projection / culling and fills these in;
	 * DrawActorLabels handles the on-screen layout (overlap avoidance, leader lines,
	 * text plates).
	 */
	struct FActorLabelCandidate
	{
		/** Text to draw on the image (typically actor display label + world position). */
		FString DisplayLabel;

		/** Pre-projected screen-space position of the actor origin. */
		int32 ScreenX = 0;
		int32 ScreenY = 0;

		/** Pixel scale for the glyph rendering — typically 2 for near, 1 for far. */
		int32 TextScale = 1;
	};

	/**
	 * Draw a multi-LOD projected world-space grid plus origin marker, axis legend,
	 * and per-intersection coordinate labels (in meters) onto Canvas.
	 *
	 * The grid is drawn in concentric LODs around the camera position so that the
	 * close-up density is high without flooding the image with distant intersections.
	 *
	 * @param Canvas              Target framebuffer.
	 * @param ViewProjectionMatrix World-to-clip matrix matching the captured viewport.
	 * @param CamLoc              Camera world-space location.
	 * @param CamFwd              Camera forward direction (unit vector).
	 * @param GridSpacing         World-space distance between grid lines (cm). Caller checks > 0 before invoking.
	 * @param GridExtent          How far the grid extends from the origin in each axis (cm).
	 * @param GridHeight          Height of the ground-plane grid in world space (Z, or "up" in LUF terms).
	 * @param MaxLabelDistance    Maximum distance (cm) for coordinate labels at intersections.
	 */
	void DrawWorldGrid(
		FBitmapCanvas& Canvas,
		const FMatrix& ViewProjectionMatrix,
		const FVector& CamLoc,
		const FVector& CamFwd,
		double GridSpacing,
		double GridExtent,
		double GridHeight,
		double MaxLabelDistance);

	/**
	 * Draw per-actor crosshair, label plate, and leader-line callouts onto Canvas.
	 *
	 * Performs a simple greedy overlap-avoidance layout — labels are bumped vertically
	 * in fixed steps to avoid colliding with already-placed labels. When a label can't
	 * sit at its default position above the crosshair, a faint leader line connects
	 * the label back to the crosshair.
	 *
	 * Candidates should already be in the order the caller wants them placed
	 * (typically nearest-to-camera first).
	 */
	void DrawActorLabels(FBitmapCanvas& Canvas, TArrayView<const FActorLabelCandidate> Candidates);

	// Projection helpers -----------------------------------------------------

	/**
	 * Project a world-space point to screen pixel coordinates.
	 * Returns false if the point is behind the camera.
	 */
	bool ProjectWorldToScreen(const FVector& WorldPos, const FMatrix& ViewProjectionMatrix, FIntPoint Size, int32& OutX, int32& OutY);

	/**
	 * Project a line segment from A to B to screen space, clipping in world space against a
	 * near plane just in front of the camera. World-space clipping (as opposed to clip-space
	 * W clipping) preserves line straightness after projection.
	 *
	 * Returns false if the segment is entirely behind the camera.
	 * Output coordinates are clamped to a safe range to prevent Bresenham overflow.
	 */
	bool ProjectLineClipped(const FVector& A, const FVector& B,
							const FMatrix& ViewProjectionMatrix, FIntPoint Size,
							int32& OutX0, int32& OutY0, int32& OutX1, int32& OutY1,
							const FVector& CamPos, const FVector& CamFwd);
}
