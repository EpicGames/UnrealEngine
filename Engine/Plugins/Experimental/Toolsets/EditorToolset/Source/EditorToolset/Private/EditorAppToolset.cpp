// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorAppToolset.h"

#include "Animation/AnimationAsset.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ContentBrowserModule.h"
#include "Containers/Ticker.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWindow.h"
#include "Editor.h"
#include "EditorAssetLibrary.h"
#include "IContentBrowserSingleton.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/AssetManager.h"
#include "Editor/GroupActor.h"
#include "Engine/LevelBounds.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SphereReflectionCapture.h"
#include "Engine/StaticMesh.h"
#include "Engine/StreamableManager.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "GameFramework/DefaultPhysicsVolume.h"
#include "GameFramework/Info.h"
#include "EngineUtils.h"
#include "HAL/ConsoleManager.h"
#include "Kismet/KismetSystemLibrary.h"
#include "IAssetViewport.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Modules/ModuleManager.h"
#include "Misc/ObjectThumbnail.h"
#include "Misc/PackageName.h"
#include "ObjectTools.h"
#include "PlayInEditorDataTypes.h"
#include "RenderingThread.h"
#include "SceneView.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Selection.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "Templates/SharedPointer.h"
#include "ToolsetRegistry/DelegateHandle.h"
#include "ToolsetRegistry/ToolCallAsyncResultImage.h"
#include "ToolsetRegistry/ToolCallAsyncResultVoid.h"
#include "ToolsetRegistry/ToolsetImage.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "UnrealClient.h"

#include "BitmapAnnotation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorAppToolset)

DEFINE_LOG_CATEGORY_STATIC(LogEditorToolset, Log, All);

namespace
{
	// Helper that renders an asset thumbnail and resolves a result.
	// Initiates an async load then polls each tick until the asset is loaded and
	// all compilation (assets and shaders) is complete before rendering.
	struct FAssetThumbnailCapture : public TSharedFromThis<FAssetThumbnailCapture>
	{
		explicit FAssetThumbnailCapture(const FString& InAssetPath) :
			AssetPath(InAssetPath),
			Result(NewObject<UToolCallAsyncResultImage>()),
			StartTime(FPlatformTime::Seconds())
		{
		}
		~FAssetThumbnailCapture() = default;

		static TStrongObjectPtr<UToolCallAsyncResultImage> Start(const FString& InAssetPath)
		{
			return MakeShared<FAssetThumbnailCapture>(InAssetPath)->StartCapture();
		}

	private:
		TStrongObjectPtr<UToolCallAsyncResultImage> StartCapture()
		{
			// Initiate async load; the ticker polls for completion.
			StreamableHandle = UAssetManager::Get().GetStreamableManager().RequestAsyncLoad(
				FSoftObjectPath(AssetPath), FStreamableDelegate());
			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(
				FTickerDelegate::CreateLambda(
					[This = AsShared().ToSharedPtr()](float) mutable -> bool
					{
						bool bContinue = This->OnTick();
						if (!bContinue) This.Reset();
						return bContinue;
					}));
			return Result;
		}

		// Returns true to keep ticking, false when done.
		bool OnTick()
		{
			if (FPlatformTime::Seconds() - StartTime > 10.0)
			{
				Result->SetError(FString::Printf(TEXT("Timed out loading asset: %s"), *AssetPath));
				return false;
			}

			// Check to see if asset is loaded.
			TObjectPtr<UObject> Asset = FSoftObjectPath(AssetPath).ResolveObject();
			if (!Asset)
			{
				return true;
			}

			// Poll until asset-level and shader compilation are both complete.
			// This matches what the editor thumbnail system does before rendering.
			const IInterface_AsyncCompilation* AsyncComp =
				Cast<IInterface_AsyncCompilation>(Asset);
			if (AsyncComp && AsyncComp->IsCompiling())
			{
				return true;
			}

			// UMaterialInstance::IsCompiling() only checks the instance's own static
			// permutation resources; check the base material separately.
			if (UMaterialInterface* MatInterface = Cast<UMaterialInterface>(Asset))
			{
				if (MatInterface->IsCompiling())
				{
					return true;
				}
				if (UMaterial* Base = MatInterface->GetMaterial(); Base && Base->IsCompiling())
				{
					return true;
				}
			}
			// Poll for texture async build and mip streaming completion.
			if (UTexture* Texture = Cast<UTexture>(Asset))
			{
				if (!Texture->IsAsyncCacheComplete() || Texture->HasPendingInitOrStreaming())
				{
					return true;
				}
			}

			// All compilation and streaming done. Release the streamable handle and render.
			StreamableHandle.Reset();
			TickerHandle.Reset();
			FlushRenderingCommands();

			FObjectThumbnail NewThumbnail;
			ThumbnailTools::RenderThumbnail(
				Asset,
				ThumbnailTools::DefaultThumbnailSize,
				ThumbnailTools::DefaultThumbnailSize,
				ThumbnailTools::EThumbnailTextureFlushMode::NeverFlush,
				nullptr,
				&NewThumbnail);

			const int32 Width = NewThumbnail.GetImageWidth();
			const int32 Height = NewThumbnail.GetImageHeight();
			const TArray<uint8>& ImageBytes = NewThumbnail.GetUncompressedImageData();
			const int32 ExpectedBytes = Width * Height * 4; // BGRA8, 4 bytes per pixel

			if (Width == 0 || Height == 0 || ImageBytes.Num() != ExpectedBytes)
			{
				Result->SetError(FString::Printf(
					TEXT("Failed to render thumbnail for: %s"), *Asset->GetPathName()));
				return false;
			}

			// The thumbnail data is raw BGRA8. Copy into FColor and swap R and B so the
			// memory layout is RGBA, then encode with ERGBFormat::RGBA (no libpng channel
			// transform needed). This avoids relying on PNG_TRANSFORM_BGR being compiled in.
			// Force alpha to 255 since some thumbnail renderers do not write the alpha channel.
			TArray<FColor> Colors;
			Colors.SetNumUninitialized(Width * Height);
			FMemory::Memcpy(Colors.GetData(), ImageBytes.GetData(), ImageBytes.Num());
			for (FColor& Color : Colors)
			{
				Swap(Color.R, Color.B);
				Color.A = 255;
			}

			FToolsetImage Image;
			if (Image.SetFromBitmap(Colors, FIntPoint(Width, Height), ERGBFormat::RGBA))
			{
				Result->SetValue(Image);
			}
			else
			{
				Result->SetError(FString::Printf(
					TEXT("Failed to encode thumbnail for: %s"), *Asset->GetPathName()));
			}
			return false;
		}

		FString AssetPath;
		TStrongObjectPtr<UToolCallAsyncResultImage> Result;
		TSharedPtr<FStreamableHandle> StreamableHandle;
		FTSTicker::FDelegateHandle TickerHandle;
		double StartTime;
	};


}

FString UEditorAppToolset::SearchCVars(const FString& Name)
{
	TSharedPtr<FJsonObject> Results = MakeShared<FJsonObject>();
	const auto OnConsoleVariable = [&Results](const TCHAR* Name, IConsoleObject* CVar)
	{
		if (TSharedPtr<FJsonObject> CVarJson = CVarToJson(CVar->AsVariable()))
		{
			Results->SetObjectField(Name, CVarJson);
		}
	};
	IConsoleManager& ConsoleManager = IConsoleManager::Get();
	ConsoleManager.ForEachConsoleObjectThatContains(FConsoleObjectVisitor::CreateLambda(OnConsoleVariable), *Name);
	FString JsonString;
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter =
		TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&JsonString);
	FJsonSerializer::Serialize(Results.ToSharedRef(), JsonWriter);
	return JsonString;
}

TSharedPtr<FJsonObject> UEditorAppToolset::CVarToJson(IConsoleObject* CVar)
{
	if (!CVar || !CVar->IsEnabled())
	{
		return nullptr;
	}

	TSharedPtr<FJsonObject> CVarData = MakeShared<FJsonObject>();
	CVarData->SetStringField(FString(TEXT("help")), CVar->GetHelp());
	if (IConsoleVariable* Variable = CVar->AsVariable())
	{
		if (Variable->IsVariableBool())
		{
			CVarData->SetBoolField(
				FString(TEXT("value")), Variable->GetBool());
		}
		else if (Variable->IsVariableInt())
		{
			CVarData->SetNumberField(
				FString(TEXT("value")), Variable->GetInt());
		}
		else if (Variable->IsVariableFloat())
		{
			CVarData->SetNumberField(
				FString(TEXT("value")), Variable->GetFloat());
		}
		else if (Variable->IsVariableString())
		{
			CVarData->SetStringField(
				FString(TEXT("value")), Variable->GetString());
		}
		else
		{
			CVarData->SetStringField(TEXT("value"), Variable->GetDefaultValue());
		}
	}
	return CVarData;
}

UToolCallAsyncResultImage* UEditorAppToolset::CaptureAssetImage(const FString& AssetPath)
{
	TObjectPtr<UToolCallAsyncResultImage> Result = NewObject<UToolCallAsyncResultImage>();
	// Normalize a bare package path to a full object path so the registry lookup
	// and async load both work. e.g. /Game/Foo/Bar  ->  /Game/Foo/Bar.Bar
	FString ObjectPath = AssetPath;
	if (!ObjectPath.Contains(TEXT(".")))
	{
		int32 LastSlash;
		if (ObjectPath.FindLastChar(TEXT('/'), LastSlash))
		{
			ObjectPath += TEXT(".") + ObjectPath.RightChop(LastSlash + 1);
		}
	}
	const FAssetData AssetData =
		IAssetRegistry::GetChecked().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
	if (!AssetData.IsValid())
	{
		Result->SetError(FString::Printf(TEXT("Asset not found: %s"), *ObjectPath));
		return Result.Get();
	}
	else if (AssetData.IsInstanceOf<UWorld>())
	{
		Result->SetError(FString::Printf(
			TEXT("CaptureAssetImage does not support levels; use CaptureViewport for the current level: %s"),
			*ObjectPath));
	}
	else if (!AssetData.IsInstanceOf<UAnimationAsset>() &&
		!AssetData.IsInstanceOf<USkeleton>() &&
		!AssetData.IsInstanceOf<UStaticMesh>() &&
		!AssetData.IsInstanceOf<USkeletalMesh>() &&
		!AssetData.IsInstanceOf<UTexture>() &&
		!AssetData.IsInstanceOf<UMaterialInterface>())
	{
		Result->SetError(FString::Printf(
			TEXT("Asset type does not support image capture: %s"), *ObjectPath));
	}
	else
	{
		Result = FAssetThumbnailCapture::Start(ObjectPath).Get();
	}
	return Result.Get();
}

TArray<AActor*> UEditorAppToolset::GetSelectedActors()
{
	TArray<AActor*> Result;
	if (!GEditor)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Editor not found."));
		return Result;
	}
	for (FSelectionIterator Iter(*GEditor->GetSelectedActors()); Iter; ++Iter)
	{
		if (AActor* Actor = Cast<AActor>(*Iter))
		{
			Result.Add(Actor);
		}
	}
	return Result;
}

void UEditorAppToolset::SelectActors(const TArray<AActor*>& Actors)
{
	if (!GEditor)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Editor not found."));
		return;
	}
	USelection* Selection = GEditor->GetSelectedActors();
	Selection->Modify();
	Selection->BeginBatchSelectOperation();
	GEditor->SelectNone(false, true, false);
	for (AActor* Actor : Actors)
	{
		if (Actor)
		{
			GEditor->SelectActor(Actor, true, false);
		}
	}
	Selection->EndBatchSelectOperation();
	GEditor->NoteSelectionChange();
}

FTransform UEditorAppToolset::GetCameraTransform()
{
	if (!GCurrentLevelEditingViewportClient)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("There is no current level camera."));
		return FTransform::Identity;
	}
	return FTransform(
		GCurrentLevelEditingViewportClient->GetViewRotation(),
		GCurrentLevelEditingViewportClient->GetViewLocation());
}

void UEditorAppToolset::SetCameraTransform(const FTransform& Transform)
{
	if (!GCurrentLevelEditingViewportClient)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("There is no current level camera."));
		return;
	}
	GCurrentLevelEditingViewportClient->SetViewLocationForOrbiting(Transform.GetLocation());
	GCurrentLevelEditingViewportClient->SetViewLocation(Transform.GetLocation());
	GCurrentLevelEditingViewportClient->SetViewRotation(Transform.Rotator());
}

void UEditorAppToolset::FocusOnActors(const TArray<AActor*>& Actors)
{
	if (!GEditor)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Editor not found."));
		return;
	}
	if (GEditor->PlayWorld)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("FocusOnActors is not supported during PIE."));
		return;
	}

	GEditor->MoveViewportCamerasToActor(Actors, true);
}

TArray<AActor*> UEditorAppToolset::GetVisibleActors()
{
	TArray<AActor*> Result;
	if (!GEditor || !GCurrentLevelEditingViewportClient || !GCurrentLevelEditingViewportClient->Viewport)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Editor not found."));
		return Result;
	}
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("No world to query."));
		return Result;
	}
	FLevelEditorViewportClient* LevelVC = GCurrentLevelEditingViewportClient;
	FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(
		LevelVC->Viewport, LevelVC->GetScene(), LevelVC->EngineShowFlags)
		.SetRealtimeUpdate(LevelVC->IsRealtime()));
	const TSharedPtr<FSceneView> SceneView = MakeShareable<>(LevelVC->CalcSceneView(&ViewFamily));
	for (TActorIterator<AActor> It(World); It; ++It)
	{
		const FBox Box = It->GetComponentsBoundingBox(/*bNonColliding=*/true);
		const bool bInFrustum = Box.IsValid
			? SceneView->ViewFrustum.IntersectBox(Box.GetCenter(), Box.GetExtent())
			: SceneView->ViewFrustum.IntersectSphere(It->GetActorLocation(), 0.f);
		if (bInFrustum)
		{
			Result.Add(*It);
		}
	}
	return Result;
}

FVector2D UEditorAppToolset::WorldPosToScreenCoords(FVector Position)
{
	if (!GEditor)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Editor not found."));
		return FVector2D::ZeroVector;
	}
	UUnrealEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
	check(Subsystem);
	FVector2D ScreenCoords;
	if (!Subsystem->WorldToScreen(Position, ScreenCoords))
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Unable to convert position into screen space."));
		return FVector2D::ZeroVector;
	}
	FIntPoint Size;
	if (!Subsystem->GetLevelViewportSize(Size))
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Unable to retrieve viewport size."));
		return FVector2D::ZeroVector;
	}
	return FVector2D(ScreenCoords.X / Size.X, ScreenCoords.Y / Size.Y);
}

FVector UEditorAppToolset::ScreenCoordsToWorld(FVector2D Coords, float TraceDistance)
{
	if (Coords.X < 0 || Coords.X > 1 || Coords.Y < 0 || Coords.Y > 1)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Invalid viewport coords (%f, %f)."), Coords.X, Coords.Y));
		return FVector::ZeroVector;
	}
	if (!GEditor)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Editor not found."));
		return FVector::ZeroVector;
	}
	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("No world to query."));
		return FVector::ZeroVector;
	}
	UUnrealEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
	check(Subsystem);
	FIntPoint Size;
	if (!Subsystem->GetLevelViewportSize(Size))
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Unable to retrieve viewport size."));
		return FVector::ZeroVector;
	}
	FVector Origin, Direction;
	const FVector2D PixelCoords(Coords.X * Size.X, Coords.Y * Size.Y);
	if (!Subsystem->ScreenToWorld(PixelCoords, Origin, Direction))
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Unable to convert screen coordinates into world space."));
		return FVector::ZeroVector;
	}
	FHitResult HitResult;
	TArray<AActor*> ActorsToIgnore;
	const bool bHit = UKismetSystemLibrary::LineTraceSingle(
		World,
		Origin,
		Origin + Direction * TraceDistance,
		ETraceTypeQuery::TraceTypeQuery1,
		true,
		ActorsToIgnore,
		EDrawDebugTrace::None,
		HitResult,
		true);
	if (!bHit)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Unable to find a solid surface at the given screen coordinates."));
		return FVector::ZeroVector;
	}
	return HitResult.Location;
}

TArray<FString> UEditorAppToolset::GetSelectedAssets()
{
	FContentBrowserModule& ContentBrowserModule =
		FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> AssetData;
	ContentBrowserModule.Get().GetAllSelectedAssets(AssetData);
	TArray<FString> Result;
	for (const FAssetData& Data : AssetData)
	{
		Result.Add(Data.PackageName.ToString());
	}
	return Result;
}

UToolCallAsyncResultVoid* UEditorAppToolset::SelectAssets(const TArray<FString>& AssetPaths)
{
	UToolCallAsyncResultVoid* Result = NewObject<UToolCallAsyncResultVoid>();

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	TSet<FName> ExpectedPackages;
	for (const FString& Path : AssetPaths)
	{
		TArray<FAssetData> Found;
		AssetRegistry.GetAssetsByPackageName(FName(*Path), Found);
		if (Found.Num() > 0)
		{
			ExpectedPackages.Add(Found[0].PackageName);
		}
	}

	UEditorAssetLibrary::SyncBrowserToObjects(AssetPaths);

	// Selection is applied asynchronously by the content browser. Poll each tick
	// until all expected assets appear in the selection or the timeout elapses.
	TStrongObjectPtr<UToolCallAsyncResultVoid> StrongResult(Result);
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
		[StrongResult, ExpectedPackages, StartTime = FPlatformTime::Seconds()](float) mutable -> bool
		{
			FContentBrowserModule& CBModule =
				FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			TArray<FAssetData> SelectedAssets;
			CBModule.Get().GetAllSelectedAssets(SelectedAssets);
			TSet<FName> SelectedPackages;
			for (const FAssetData& Data : SelectedAssets)
			{
				SelectedPackages.Add(Data.PackageName);
			}
			bool bAllSelected = true;
			for (const FName& Expected : ExpectedPackages)
			{
				if (!SelectedPackages.Contains(Expected))
				{
					bAllSelected = false;
					break;
				}
			}
			if (bAllSelected)
			{
				StrongResult->SetCompleted();
				StrongResult.Reset();
				return false;
			}
			const float MaxSelectionTime = 5.f;
			if (FPlatformTime::Seconds() - StartTime > MaxSelectionTime)
			{
				StrongResult->SetError(TEXT("Timed out waiting for asset selection."));
				StrongResult.Reset();
				return false;
			}
			return true; // keep polling
		}));

	return Result;
}

FString UEditorAppToolset::GetContentBrowserPath()
{
	FContentBrowserModule& ContentBrowserModule =
		FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const FContentBrowserItemPath CurrentPath = ContentBrowserModule.Get().GetCurrentPath();
	return CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : FString();
}

void UEditorAppToolset::SetContentBrowserPath(const FString& Path)
{
	FContentBrowserModule& ContentBrowserModule =
		FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SetSelectedPaths({Path});
	const FString ResultPath = UEditorAppToolset::GetContentBrowserPath();
	if (ResultPath != Path)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Failed to navigate content browser to '%s': ended up at '%s'."),
			*Path, *ResultPath));
	}
}

void UEditorAppToolset::OpenEditorForAsset(const FString& AssetPath)
{
	if (!GEditor)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Editor not found."));
		return;
	}
	UObject* Asset = StaticLoadObject(UObject::StaticClass(), nullptr, *AssetPath);
	if (!Asset)
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("%s is not a valid asset path."), *AssetPath));
		return;
	}
	UAssetEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	check(Subsystem);
	if (!Subsystem->OpenEditorForAsset(Asset))
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Failed to open editor for asset: %s"), *AssetPath));
	}
}

TArray<FString> UEditorAppToolset::GetOpenAssets()
{
	TArray<FString> Result;
	if (!GEditor)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Editor not found."));
		return Result;
	}
	UAssetEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	check(Subsystem);
	for (UObject* Asset : Subsystem->GetAllEditedAssets())
	{
		if (Asset)
		{
			Result.Add(FPackageName::ObjectPathToPackageName(Asset->GetPathName()));
		}
	}
	return Result;
}

FToolsetImage UEditorAppToolset::CaptureEditorImage()
{
	// Capture every visible Slate window (main window, asset editors, detached panels, etc.)
	// back-to-front, then composite them into a single image using their screen positions.
	struct FCapturedWindow
	{
		FSlateRect ScreenRect;
		TArray<FColor> Colors;
		FIntVector Size;
	};

	TArray<TSharedRef<SWindow>> Windows;
	FSlateApplication::Get().GetAllVisibleWindowsOrdered(Windows);

	TArray<FCapturedWindow> Captured;
	float MinX = TNumericLimits<float>::Max();
	float MinY = TNumericLimits<float>::Max();
	float MaxX = TNumericLimits<float>::Lowest();
	float MaxY = TNumericLimits<float>::Lowest();

	for (const TSharedRef<SWindow>& Window : Windows)
	{
		FCapturedWindow& Entry = Captured.AddDefaulted_GetRef();
		Entry.ScreenRect = Window->GetRectInScreen();
		if (!FSlateApplication::Get().TakeScreenshot(Window, Entry.Colors, Entry.Size) ||
			Entry.Colors.IsEmpty())
		{
			Captured.Pop();
			continue;
		}
		MinX = FMath::Min(MinX, Entry.ScreenRect.Left);
		MinY = FMath::Min(MinY, Entry.ScreenRect.Top);
		MaxX = FMath::Max(MaxX, Entry.ScreenRect.Left + Entry.Size.X);
		MaxY = FMath::Max(MaxY, Entry.ScreenRect.Top + Entry.Size.Y);
	}

	if (Captured.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Failed to capture any editor windows."));
		return FToolsetImage{};
	}

	const FIntPoint TotalSize(
		FMath::Max(1, FMath::RoundToInt(MaxX - MinX)),
		FMath::Max(1, FMath::RoundToInt(MaxY - MinY)));

	// Blit each window into the canvas at its screen-relative offset, back-to-front.
	TArray<FColor> Canvas;
	Canvas.SetNumZeroed(TotalSize.X * TotalSize.Y);
	for (const FCapturedWindow& Win : Captured)
	{
		const int32 OffsetX = FMath::RoundToInt(Win.ScreenRect.Left - MinX);
		const int32 OffsetY = FMath::RoundToInt(Win.ScreenRect.Top - MinY);
		for (int32 Row = 0; Row < Win.Size.Y; ++Row)
		{
			const int32 CanvasY = OffsetY + Row;
			if (CanvasY < 0 || CanvasY >= TotalSize.Y) continue;
			const int32 SrcX0 = FMath::Max(0, -OffsetX);
			const int32 SrcX1 = FMath::Min(Win.Size.X, TotalSize.X - OffsetX);
			if (SrcX0 >= SrcX1) continue;
			FMemory::Memcpy(
				&Canvas[CanvasY * TotalSize.X + OffsetX + SrcX0],
				&Win.Colors[Row * Win.Size.X + SrcX0],
				(SrcX1 - SrcX0) * sizeof(FColor));
		}
	}

	FIntPoint Dimensions = TotalSize;
	TArray<FColor> ScaledPixels;
	const int32 MaxImageSize = 1280;
	if (Dimensions.X > MaxImageSize || Dimensions.Y > MaxImageSize)
	{
		const float Scale = (Dimensions.X >= Dimensions.Y)
			? (float)MaxImageSize / Dimensions.X
			: (float)MaxImageSize / Dimensions.Y;
		const FIntPoint Scaled(
			FMath::Max(1, (int32)(Dimensions.X * Scale)),
			FMath::Max(1, (int32)(Dimensions.Y * Scale)));
		ScaledPixels.SetNumUninitialized(Scaled.X * Scaled.Y);
		for (int32 Y = 0; Y < Scaled.Y; ++Y)
		{
			for (int32 X = 0; X < Scaled.X; ++X)
			{
				ScaledPixels[Y * Scaled.X + X] =
					Canvas[(int32)(Y / Scale) * Dimensions.X + (int32)(X / Scale)];
			}
		}
		Dimensions = Scaled;
	}

	FToolsetImage Image;
	const TArray<FColor>& Final = ScaledPixels.Num() > 0 ? ScaledPixels : Canvas;
	if (!Image.SetFromBitmap(Final, Dimensions))
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Failed to encode editor screenshot."));
		return FToolsetImage{};
	}
	return Image;
}

namespace
{
	// Waits for PostPIEStarted to fire (signalling BeginPlay has been called),
	// then waits an additional warmup period before completing. The warmup is a
	// heuristic: agents use it to let game-level initialization (services,
	// authentication, plugin warmup) settle before inspecting state or logs.
	class FPIEStartupWatcher : public TSharedFromThis<FPIEStartupWatcher>
	{
	public:
		static void Start(TNotNull<UToolCallAsyncResultVoid*> InResult, double InWarmupSeconds, double InTimeoutSeconds)
		{
			TSharedRef<FPIEStartupWatcher> Watcher = MakeShared<FPIEStartupWatcher>();
			Watcher->Result = TStrongObjectPtr<UToolCallAsyncResultVoid>(InResult);
			Watcher->WarmupSeconds = InWarmupSeconds;
			Watcher->TimeoutSeconds = InTimeoutSeconds;
			Watcher->RequestTime = FPlatformTime::Seconds();

			// Subscribe before kicking the play request to avoid missing the event.
			Watcher->PIEStartedHandle = UE::ToolsetRegistry::FDelegateHandleRaii::Create(
				FEditorDelegates::PostPIEStarted,
				FEditorDelegates::PostPIEStarted.AddLambda(
					[WeakWatcher = Watcher->AsWeak()](const bool /*bIsSimulating*/) mutable
					{
						if (TSharedPtr<FPIEStartupWatcher> Pinned = WeakWatcher.Pin())
						{
							Pinned->PIEStartedTime = FPlatformTime::Seconds();
						}
					}));

			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[This = Watcher.ToSharedPtr()](float) mutable -> bool
				{
					bool bContinue = This->OnTick();
					if (!bContinue)
					{
						This.Reset();
					}
					return bContinue;
				}));
		}

	private:
		// Returns true to keep ticking, false when done.
		bool OnTick()
		{
			const double Now = FPlatformTime::Seconds();

			if (Now - RequestTime > TimeoutSeconds)
			{
				Result->SetError(TEXT("Timed out waiting for PIE to start."));
				return false;
			}

			// PostPIEStarted hasn't fired yet — keep waiting.
			if (!PIEStartedTime.IsSet())
			{
				return true;
			}

			// PIE started, then ended before warmup completed.
			if (!GEditor || !GEditor->PlayWorld)
			{
				Result->SetError(TEXT("PIE ended before warmup completed."));
				return false;
			}

			if (Now - PIEStartedTime.GetValue() < WarmupSeconds)
			{
				return true;
			}

			Result->SetCompleted();
			return false;
		}

	private:
		// The result object passed in externally. We hold a strong ref to it so it doesn't get GC'd.
		TStrongObjectPtr<UToolCallAsyncResultVoid> Result;
		UE::ToolsetRegistry::FDelegateHandleRaii PIEStartedHandle;
		double RequestTime = 0.0;
		TOptional<double> PIEStartedTime;
		double WarmupSeconds = 0.0;
		double TimeoutSeconds = 30.0;
	};

	// Waits for ShutdownPIE to fire (PIE session has completely shutdown - the
	// symmetric counterpart to PostPIEStarted) and then completes the result.
	class FPIEShutdownWatcher : public TSharedFromThis<FPIEShutdownWatcher>
	{
	public:
		static void Start(TNotNull<UToolCallAsyncResultVoid*> InResult, double InTimeoutSeconds)
		{
			TSharedRef<FPIEShutdownWatcher> Watcher = MakeShared<FPIEShutdownWatcher>();
			Watcher->Result = TStrongObjectPtr<UToolCallAsyncResultVoid>(InResult);
			Watcher->TimeoutSeconds = InTimeoutSeconds;
			Watcher->RequestTime = FPlatformTime::Seconds();

			// Subscribe before kicking the end request to avoid missing the event.
			Watcher->ShutdownHandle = UE::ToolsetRegistry::FDelegateHandleRaii::Create(
				FEditorDelegates::ShutdownPIE,
				FEditorDelegates::ShutdownPIE.AddLambda(
					[WeakWatcher = Watcher->AsWeak()](const bool /*bIsSimulating*/) mutable
					{
						if (TSharedPtr<FPIEShutdownWatcher> Pinned = WeakWatcher.Pin())
						{
							Pinned->bShutdownFired = true;
						}
					}));

			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[This = Watcher.ToSharedPtr()](float) mutable -> bool
				{
					bool bContinue = This->OnTick();
					if (!bContinue)
					{
						This.Reset();
					}
					return bContinue;
				}));
		}

	private:
		// Returns true to keep ticking, false when done.
		bool OnTick()
		{
			if (bShutdownFired)
			{
				Result->SetCompleted();
				return false;
			}
			if (FPlatformTime::Seconds() - RequestTime > TimeoutSeconds)
			{
				Result->SetError(TEXT("Timed out waiting for PIE to stop."));
				return false;
			}
			return true;
		}

	private:
		// The result object passed in externally. We hold a strong ref to it so it doesn't get GC'd.
		TStrongObjectPtr<UToolCallAsyncResultVoid> Result;
		UE::ToolsetRegistry::FDelegateHandleRaii ShutdownHandle;
		double RequestTime = 0.0;
		double TimeoutSeconds = 30.0;
		bool bShutdownFired = false;
	};

	// Shared setup for StartPIE. Assumes the caller has verified GEditor is valid
	// and no session is active. The watcher subscribes to FEditorDelegates::PostPIEStarted,
	// which fires for in-process PIE only, so out-of-process editor play modes
	// are downgraded to in-viewport with a warning.
	void StartPlaySession(TNotNull<UToolCallAsyncResultVoid*> Result, const FPIESessionOptions& Options)
	{
		FPIEStartupWatcher::Start(Result, /*WarmupSeconds=*/FMath::Max(0.f, Options.WarmupSeconds), /*TimeoutSeconds=*/30.0);

		FRequestPlaySessionParams Params;
		Params.SessionDestination = EPlaySessionDestinationType::InProcess;
		Params.WorldType = Options.bSimulate
			? EPlaySessionWorldType::SimulateInEditor
			: EPlaySessionWorldType::PlayInEditor;

		// Caller-supplied transform overrides the GameMode's default spawn logic.
		if (Options.StartTransform.IsSet())
		{
			Params.StartLocation = Options.StartTransform->GetLocation();
			Params.StartRotation = Options.StartTransform->Rotator();
		}

		// Attach to the active level viewport so the session plays inside the editor
		// viewport instead of spawning a floating window. Mirrors the editor's own
		// Play button (FLevelEditorModule::StartPlayInEditorSession).
		TSharedPtr<IAssetViewport> ActiveLevelViewport;
		if (FModuleManager::Get().IsModuleLoaded(TEXT("LevelEditor")))
		{
			FLevelEditorModule& LevelEditorModule =
				FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
			ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();
		}

		switch (Options.PlayMode.GetValue())
		{
		case PlayMode_InEditorFloating:
			// Caller asked for a floating window — leave DestinationSlateViewport unset.
			break;

		case PlayMode_InNewProcess:
		case PlayMode_InMobilePreview:
		case PlayMode_InVR:
		case PlayMode_QuickLaunch:
			UE_LOG(LogEditorToolset, Warning, TEXT("Requested play mode is out-of-process and incompatible with ")
				TEXT("this tool (the watcher needs in-process PIE). Falling back to Play In Viewport."));
			// fall through to attach to viewport
		default:
			if (ActiveLevelViewport.IsValid())
			{
				Params.DestinationSlateViewport = ActiveLevelViewport;
			}
			break;
		}

		GEditor->RequestPlaySession(Params);
	}
}

UToolCallAsyncResultVoid* UEditorAppToolset::StartPIE(const FPIESessionOptions& Options)
{
	UToolCallAsyncResultVoid* Result = NewObject<UToolCallAsyncResultVoid>();
	if (!GEditor)
	{
		Result->SetError(TEXT("Editor not found."));
		return Result;
	}
	if (GEditor->PlayWorld)
	{
		Result->SetError(TEXT("A play session is already running."));
		return Result;
	}

	StartPlaySession(Result, Options);
	return Result;
}

UToolCallAsyncResultVoid* UEditorAppToolset::StopPIE()
{
	UToolCallAsyncResultVoid* Result = NewObject<UToolCallAsyncResultVoid>();
	if (!GEditor)
	{
		Result->SetError(TEXT("Editor not found."));
		return Result;
	}
	if (!GEditor->PlayWorld)
	{
		Result->SetError(TEXT("A play session is not currently running."));
		return Result;
	}

	FPIEShutdownWatcher::Start(Result, /*TimeoutSeconds=*/30.0);
	GEditor->RequestEndPlayMap();
	return Result;
}

bool UEditorAppToolset::IsPIERunning()
{
	return GEditor && GEditor->PlayWorld != nullptr;
}

namespace
{
	using namespace UE::ToolsetRegistry::BitmapAnnotation;

	/**
	 * Build the view-projection matrix for the given viewport client using the
	 * engine-accurate FSceneView::ViewMatrices.GetWorldToClip() so projected points
	 * match exactly what the user sees on screen.
	 *
	 * Returns an empty optional if no scene is available (typically headless / no
	 * rendering): a captured framebuffer with overlay labels projected against a
	 * fabricated matrix would be misleading to a vision agent, so callers should
	 * surface that as a tool error rather than draw garbage.
	 */
	TOptional<FMatrix> GetAnnotationViewProjectionMatrix(FEditorViewportClient* ViewportClient)
	{
		if (!ViewportClient || !ViewportClient->Viewport || !ViewportClient->GetScene())
		{
			return {};
		}

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			ViewportClient->Viewport, ViewportClient->GetScene(), ViewportClient->EngineShowFlags));
		FSceneView* SceneView = ViewportClient->CalcSceneView(&ViewFamily);
		if (!SceneView)
		{
			return {};
		}
		// Copy out while SceneView and ViewFamily are still alive.
		return SceneView->ViewMatrices.GetWorldToClip();
	}

	/**
	 * Returns true for actors that exist in most levels but aren't meaningful to label
	 * for a vision agent: world settings, nav data, game mode, game session, light /
	 * physics / volume markers, etc.
	 *
	 * The strategy is layered so renames of individual engine classes don't silently
	 * fall through:
	 *   1. AInfo subclasses are conceptually "info" / management actors (WorldSettings,
	 *      GameMode, GameState, PlayerState, GameSession, GameNetworkManager,
	 *      NavigationSystemBase, etc.) — Epic groups these explicitly.
	 *   2. Actors with no UPrimitiveComponent have nothing to look at — pure logic /
	 *      data actors (LevelScriptActor, AtmosphericFog, etc.).
	 *   3. A small explicit allow-list catches stragglers that have primitives but
	 *      are still uninteresting to the agent (level bounds, default physics volume,
	 *      reflection captures, group actors).
	 */
	bool IsManagementActor(AActor* Actor)
	{
		if (!Actor)
		{
			return true;
		}

		if (Actor->IsA<AInfo>())
		{
			return true;
		}

		TArray<UPrimitiveComponent*> Primitives;
		Actor->GetComponents<UPrimitiveComponent>(Primitives);
		if (Primitives.IsEmpty())
		{
			return true;
		}

		static const UClass* const ManagementClasses[] = {
			ALevelBounds::StaticClass(),
			ADefaultPhysicsVolume::StaticClass(),
			ASphereReflectionCapture::StaticClass(),
			AGroupActor::StaticClass(),
			ALevelScriptActor::StaticClass(), // Designer could attach a primitive; never user-meaningful regardless.
		};
		for (const UClass* ManagementClass : ManagementClasses)
		{
			if (Actor->IsA(ManagementClass))
			{
				return true;
			}
		}
		return false;
	}

}

FViewportCapture UEditorAppToolset::CaptureViewport(
	TOptional<FTransform> CaptureTransform,
	TOptional<FViewportAnnotationConfig> Annotations,
	bool bShowUI)
{
	using namespace UE::ToolsetRegistry::BitmapAnnotation;

	FViewportCapture Out;

	// --- 1. Find the active level viewport and capture its framebuffer ----------------------
	// Use GCurrentLevelEditingViewportClient to match the idiom used elsewhere in this
	// toolset (GetCameraTransform / GetVisibleActors) — it's the currently-focused level
	// editing viewport client, which is what the user actually sees.
	FEditorViewportClient* ViewportClient = GCurrentLevelEditingViewportClient;
	if (!ViewportClient || !ViewportClient->Viewport)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("There is no current level camera."));
		return Out;
	}

	FViewport* Viewport = ViewportClient->Viewport;
	const FIntPoint Size = Viewport->GetSizeXY();
	if (Size.X <= 0 || Size.Y <= 0)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Viewport has zero size."));
		return Out;
	}

	// Save the viewport state we may modify (camera pose + UI show flags). We use the
	// low-level SetViewLocation / SetViewRotation rather than SetViewLocationForOrbiting so
	// the orbit pivot isn't disturbed — restore is then a clean round-trip. ON_SCOPE_EXIT
	// guarantees restoration on every return path, including failures.
	const FVector SavedLocation = ViewportClient->GetViewLocation();
	const FRotator SavedRotation = ViewportClient->GetViewRotation();
	const bool bSavedModeWidgets = ViewportClient->EngineShowFlags.ModeWidgets != 0;
	const bool bSavedSelectionOutline = ViewportClient->EngineShowFlags.SelectionOutline != 0;
	const bool bSavedSelection = ViewportClient->EngineShowFlags.Selection != 0;

	const bool bModifyTransform = CaptureTransform.IsSet();
	const bool bModifyShowFlags = !bShowUI;

	if (bModifyTransform)
	{
		ViewportClient->SetViewLocation(CaptureTransform->GetLocation());
		ViewportClient->SetViewRotation(CaptureTransform->Rotator());
	}
	if (bModifyShowFlags)
	{
		ViewportClient->EngineShowFlags.SetModeWidgets(false);
		ViewportClient->EngineShowFlags.SetSelectionOutline(false);
		ViewportClient->EngineShowFlags.SetSelection(false);
	}

	ON_SCOPE_EXIT
	{
		if (bModifyTransform)
		{
			ViewportClient->SetViewLocation(SavedLocation);
			ViewportClient->SetViewRotation(SavedRotation);
		}
		if (bModifyShowFlags)
		{
			ViewportClient->EngineShowFlags.SetModeWidgets(bSavedModeWidgets);
			ViewportClient->EngineShowFlags.SetSelectionOutline(bSavedSelectionOutline);
			ViewportClient->EngineShowFlags.SetSelection(bSavedSelection);
		}
	};

	// GetViewportScreenShot reads the *current* framebuffer, so any modification to camera
	// pose or show flags requires a synchronous redraw before the read.
	if (bModifyTransform || bModifyShowFlags)
	{
		Viewport->Draw();
	}

	TArray<FColor> Bitmap;
	if (!GetViewportScreenShot(Viewport, Bitmap))
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Failed to capture viewport framebuffer."));
		return Out;
	}

	// GetViewportScreenShot returns pixels with varying alpha; force opaque so the PNG output
	// is a normal RGB image rather than something with transparent holes.
	for (FColor& Pixel : Bitmap)
	{
		Pixel.A = 255;
	}

	// Camera fields reflect the capture pose, not the (about-to-be-restored) live pose.
	Out.CameraLocation = ViewportClient->GetViewLocation();
	Out.CameraRotation = ViewportClient->GetViewRotation();
	Out.CameraFOV      = ViewportClient->ViewFOV;

	// Plain capture: skip annotation overlay, encode the framebuffer as-is, and return.
	if (!Annotations.IsSet())
	{
		if (!Out.Image.SetFromBitmap(Bitmap, Size))
		{
			UKismetSystemLibrary::RaiseScriptError(TEXT("Failed to encode viewport screenshot."));
		}
		return Out;
	}

	const FViewportAnnotationConfig& Config = Annotations.GetValue();
	Out.Grid.SpacingCm = Config.GridSpacing;
	Out.Grid.ExtentCm = Config.GridExtent;
	Out.Grid.Height = Config.GridHeight;

	// Wrap the framebuffer in a canvas so drawing primitives don't need to re-thread
	// Bitmap + Size through every call.
	FBitmapCanvas Canvas(Bitmap, Size);

	// --- 2. Build the view-projection matrix used for all overlay projections ---------------
	// Bail out cleanly if there's no usable scene (typically headless): rendering overlay
	// labels against a fabricated VP matrix would produce a confusing "almost-right" image.
	const TOptional<FMatrix> ViewProjOpt = GetAnnotationViewProjectionMatrix(ViewportClient);
	if (!ViewProjOpt.IsSet())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Annotated viewport capture requires a renderable scene; none is available."));
		return Out;
	}
	const FMatrix& ViewProj = ViewProjOpt.GetValue();
	const FVector CamLoc = Out.CameraLocation;
	const FVector CamFwd = Out.CameraRotation.Vector();

	// --- 3. Draw the multi-LOD 3D grid ------------------------------------------------------
	// GridSpacing <= 0 disables the grid entirely.
	if (Config.GridSpacing > 0.f)
	{
		DrawWorldGrid(Canvas, ViewProj, CamLoc, CamFwd,
			Config.GridSpacing, Config.GridExtent, Config.GridHeight, Config.MaxLabelDistance);
	}

	// --- 4. Collect labeled actors and lay out their text ----------------------------------
	// MaxLabelDistance <= 0 disables actor labels entirely.
	if (Config.MaxLabelDistance > 0.f)
	{
		UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
		if (World)
		{
			// Collect candidates from the actor iterator. FActorEntry is intentionally a
			// superset of BitmapAnnotation::FActorLabelCandidate: the renderer only needs
			// what's drawn (text + screen pos + scale), while we also need Name/Class/
			// WorldPos/Distance to populate the public FViewportLabel output. Keeping
			// the rendering struct minimal lets it stay decoupled from FViewportLabel.
			struct FActorEntry
			{
				FString Name;
				FString DisplayLabel;
				UClass* Class = nullptr;
				FVector WorldPos = FVector::ZeroVector;
				int32 ScreenX = 0;
				int32 ScreenY = 0;
				float Distance = 0.f;
				int32 TextScale = 1;
			};
			TArray<FActorEntry> Entries;

			for (TActorIterator<AActor> It(World); It; ++It)
			{
				AActor* Actor = *It;
				if (!Actor || Actor->IsPendingKillPending() || Actor->IsHiddenEd())
				{
					continue;
				}
				if (Actor->HasAnyFlags(RF_Transient))
				{
					continue;
				}

				UClass* ActorClass = Actor->GetClass();

				if (Config.ClassFilter && !ActorClass->IsChildOf(Config.ClassFilter))
				{
					continue;
				}
				if (IsManagementActor(Actor))
				{
					continue;
				}

				const FVector ActorLocation = Actor->GetActorLocation();
				const float Distance = FVector::Dist(CamLoc, ActorLocation);
				if (Distance > Config.MaxLabelDistance)
				{
					continue;
				}

				int32 ScreenX, ScreenY;
				if (!ProjectWorldToScreen(ActorLocation, ViewProj, Size, ScreenX, ScreenY))
				{
					continue;
				}
				// Allow a small off-screen margin so partially-off labels can still anchor.
				if (ScreenX < -100 || ScreenX >= Size.X + 100 || ScreenY < -50 || ScreenY >= Size.Y + 50)
				{
					continue;
				}

				// Preserve the canonical (level-unique) name as identity; the drawn label
				// uses the friendlier GetActorLabel if available, but that isn't unique so
				// we don't rely on it for reference.
				const FString ActorName = Actor->GetName();
				FString DisplayLabel = Actor->GetActorLabel();
				if (DisplayLabel.IsEmpty())
				{
					DisplayLabel = ActorName;
				}
				if (DisplayLabel.Len() > 30)
				{
					DisplayLabel = DisplayLabel.Left(27) + TEXT("...");
				}

				const FString PosStr = FString::Printf(TEXT(" @(%d,%d,%d)"),
					FMath::RoundToInt32(ActorLocation.X / 100.0),
					FMath::RoundToInt32(ActorLocation.Y / 100.0),
					FMath::RoundToInt32(ActorLocation.Z / 100.0));

				FActorEntry Entry;
				Entry.Name         = ActorName;
				Entry.DisplayLabel = DisplayLabel + PosStr;
				Entry.Class        = ActorClass;
				Entry.WorldPos     = ActorLocation;
				Entry.ScreenX      = ScreenX;
				Entry.ScreenY      = ScreenY;
				Entry.Distance     = Distance;
				Entry.TextScale    = (Distance < 2000.f) ? 2 : 1;
				Entries.Add(MoveTemp(Entry));
			}

			// Nearest actors get first dibs on label placement.
			Entries.Sort([](const FActorEntry& A, const FActorEntry& B)
			{
				return A.Distance < B.Distance;
			});

			// Cap labels to keep the image readable and bound the layout work.
			// Caller can override via Config.MaxLabels; <= 0 disables the cap.
			if (Config.MaxLabels > 0 && Entries.Num() > Config.MaxLabels)
			{
				Entries.SetNum(Config.MaxLabels);
			}

			// Build candidate list for the rendering pass and emit public output entries.
			TArray<FActorLabelCandidate> Candidates;
			Candidates.Reserve(Entries.Num());
			Out.LabeledActors.Reserve(Out.LabeledActors.Num() + Entries.Num());

			for (const FActorEntry& Entry : Entries)
			{
				FActorLabelCandidate Cand;
				Cand.DisplayLabel = Entry.DisplayLabel;
				Cand.ScreenX      = Entry.ScreenX;
				Cand.ScreenY      = Entry.ScreenY;
				Cand.TextScale    = Entry.TextScale;
				Candidates.Add(MoveTemp(Cand));

				FViewportLabel OutLabel;
				OutLabel.Name           = Entry.Name;
				OutLabel.Label          = Entry.DisplayLabel;
				OutLabel.Class          = Entry.Class;
				OutLabel.ScreenPosition = FIntPoint(Entry.ScreenX, Entry.ScreenY);
				OutLabel.WorldLocation  = Entry.WorldPos;
				OutLabel.DistanceCm     = Entry.Distance;
				Out.LabeledActors.Add(MoveTemp(OutLabel));
			}

			DrawActorLabels(Canvas, Candidates);
		}
	}

	// --- 5. Encode result -------------------------------------------------------------------
	if (!Out.Image.SetFromBitmap(Bitmap, Size))
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Failed to encode annotated viewport screenshot."));
		return Out;
	}

	return Out;
}
