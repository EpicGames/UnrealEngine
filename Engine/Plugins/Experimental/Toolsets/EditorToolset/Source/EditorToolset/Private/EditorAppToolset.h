// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Templates/SubclassOf.h"

#include "ToolsetRegistry/ToolsetImage.h"
#include "ToolsetRegistry/ToolsetDefinition.h"

#include "EditorAppToolset.generated.h"

class AActor;
class FJsonObject;
class IConsoleObject;
class UObject;
class UToolCallAsyncResultImage;
class UToolCallAsyncResultVoid;

/*
 * Options bag for StartPIE. All fields are optional; defaults run a standard
 * Play-In-Editor session inside the active level viewport, with a 2-second
 * warmup after PostPIEStarted to let project initialization settle.
 */
USTRUCT()
struct FPIESessionOptions
{
	GENERATED_BODY()

	/*
	 * If true, starts Simulate-In-Editor: the world ticks and AI / physics /
	 * subsystems run, but no player pawn is spawned or possessed. Useful for
	 * observing initialization, autonomous agents, and physics state
	 * independently of player input.
	 * If false (default), starts standard PIE with the player pawn possessed.
	 */
	UPROPERTY()
	bool bSimulate = false;

	/*
	 * Editor play mode (in-viewport, floating window, etc.). Out-of-process
	 * modes (NewProcess, MobilePreview, VR, QuickLaunch) are downgraded to
	 * PlayMode_InViewPort with a log warning since this tool requires
	 * in-process PIE for delegate-based completion tracking.
	 */
	UPROPERTY()
	TEnumAsByte<EPlayModeType> PlayMode = PlayMode_InViewPort;

	/*
	 * Optional spawn override. When set, the player pawn (PIE) or simulation
	 * reference (Simulate) spawns at this transform instead of the level's
	 * PlayerStart / GameMode default. To spawn at the editor viewport camera,
	 * retrieve the current editor camera properties and use the result here.
	 */
	UPROPERTY()
	TOptional<FTransform> StartTransform;

	/*
	 * Seconds to wait after the engine fires PostPIEStarted (BeginPlay called)
	 * before this call completes. Heuristic for project-specific initialization
	 * (services, authentication, plugin warmup) to settle before the agent
	 * inspects state or logs. Pass 0 to complete as soon as PIE is up.
	 */
	UPROPERTY()
	float WarmupSeconds = 2.f;
};

/**
 * One labeled actor in a viewport capture's annotation overlay.
 *
 * Name is the canonical object name (unique within the level and stable across captures);
 * Label is the human-readable text drawn on the image (actor label + position).
 * Use Name when referring back to a specific actor, Label for display.
 */
USTRUCT(BlueprintType)
struct FViewportLabel
{
	GENERATED_BODY()

	/** Canonical, level-unique name of the actor (AActor::GetName). Use this to reference the actor later. */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	FString Name;

	/** Display label that was drawn on the image (actor label + position in meters). Not guaranteed unique. */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	FString Label;

	/** Class of the actor. */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	TSubclassOf<AActor> Class;

	/** Screen-space pixel position of the actor origin in the returned image. */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	FIntPoint ScreenPosition = FIntPoint(0, 0);

	/** World-space location of the actor origin (cm). */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	FVector WorldLocation = FVector::ZeroVector;

	/** Distance from the camera to the actor origin (cm). */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	float DistanceCm = 0.f;
};

/** Grid parameters echoed back from an annotated viewport capture for round-tripping / reporting. */
USTRUCT(BlueprintType)
struct FViewportGrid
{
	GENERATED_BODY()

	/** World-space distance between grid lines (cm). Zero if the grid was disabled. */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	float SpacingCm = 0.f;

	/** How far the grid extends from the origin in each axis (cm). */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	float ExtentCm = 0.f;

	/** Height of the ground-plane grid in world space. */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	float Height = 0.f;
};

/**
 * Annotation overlay configuration for CaptureViewport.
 */
USTRUCT()
struct FViewportAnnotationConfig
{
	GENERATED_BODY()

	/** World-space distance between grid lines (cm). Pass 0 to disable the grid overlay while still drawing actor labels. */
	UPROPERTY()
	float GridSpacing = 500.f;

	/** How far the grid extends from the origin in each axis (cm). */
	UPROPERTY()
	float GridExtent = 5000.f;

	/** Height of the ground-plane grid in world space. */
	UPROPERTY()
	float GridHeight = 0.f;

	/** Maximum distance (cm) for labeling actors and grid coordinates. Pass 0 to disable actor labels while still drawing the grid. */
	UPROPERTY()
	float MaxLabelDistance = 5000.f;

	/** If non-null, only actors of this class (or a subclass) are labeled. */
	UPROPERTY()
	TSubclassOf<AActor> ClassFilter = nullptr;

	/** Maximum number of actor labels to draw. Nearest-to-camera wins. Don't draw too many or the image becomes incoherent. */
	UPROPERTY()
	int32 MaxLabels = 12;
};

/**
 * Describes the results of a viewport capture. Includes a screenshot of the level viewport plus structured metadata describing the
 * camera. When annotations are enabled, the associated metadata will be populated and the image will include a projected world-space
 * grid and per-actor labels.
 */
USTRUCT(BlueprintType)
struct FViewportCapture
{
	GENERATED_BODY()

	/** The captured screenshot (PNG encoded as base64). Empty on failure. Pixel dimensions are available via FToolsetImage. */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	FToolsetImage Image;

	/** Level viewport camera location (world space, cm). */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	FVector CameraLocation = FVector::ZeroVector;

	/** Level viewport camera rotation (world space). */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	FRotator CameraRotation = FRotator::ZeroRotator;

	/** Level viewport camera field of view, in degrees. */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	float CameraFOV = 0.f;

	/** Grid parameters used for the annotation. SpacingCm is zero when the grid was disabled. */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	FViewportGrid Grid;

	/** One entry per labeled actor in the image. */
	UPROPERTY(BlueprintReadOnly, Category = "EditorAppToolset")
	TArray<FViewportLabel> LabeledActors;
};

/// Tools for querying and modifying Unreal Editor state: console variables, asset imaging,
/// actor and asset selection, viewport camera, content browser navigation, and Play-In-Editor
/// session control.
UCLASS(MinimalAPI)
class UEditorAppToolset : public UToolsetDefinition
{
	GENERATED_BODY()
public:
	/*
	 * Finds all console variables that contain a given name.
	 * @param Name The partial or full name to search for.
	 * @return A JSON dict with info about matching cvars.
	 */
	UFUNCTION(meta = (AICallable))
	static FString SearchCVars(const FString& Name);

	/*
	 * Renders a thumbnail for the specified asset (e.g. static meshes, skeletal meshes,
	 * skeletons, animations, montages, materials, textures).
	 * @param AssetPath The path to the asset, e.g. '/Game/Meshes/SM_Cube'.
	 * @return An image of the asset.
	 */
	UFUNCTION(meta = (AICallable))
	static UToolCallAsyncResultImage* CaptureAssetImage(const FString& AssetPath);

	/*
	 * Captures an image of the entire editor application as the user sees it.
	 * @return An image of editor windows as they appear on the users' desktop.
	 */
	UFUNCTION(meta = (AICallable))
	static FToolsetImage CaptureEditorImage();

	/*
	 * Captures the level viewport with optional annotations.
	 * 
	 * Annotations rendering overlays a projected 3D world-space grid plus
	 * name + position labels on visible actors. The grid is drawn at a configurable
	 * ground-plane Z and projected through the camera, with coordinate numbers at
	 * intersections (shown in meters). Each labeled actor gets a crosshair at its
	 * projected screen position with a leader-line callout placed to avoid overlap. This
	 * gives a vision-capable agent spatial awareness: it can reference grid
	 * coordinates to direct placement and identify scene contents by label.
	 *
	 * @param CaptureTransform Optional pose to capture from. If unset, uses the viewport's
	 *   current camera.
	 * @param Annotations Optional annotation overlay configuration. Only use this when
	 *   you need the information in order to perform spatial actions.
	 * @param bShowUI If false (default), editor UI overlays such as transform gizmos and
	 *   selection outlines are hidden in the captured image. Set true to capture exactly
	 *   what's on screen, gizmos and all.
	 * @return The captured image and associated metadata.
	 */
	UFUNCTION(meta = (AICallable), Category = "EditorAppToolset")
	static FViewportCapture CaptureViewport(
		TOptional<FTransform> CaptureTransform,
		TOptional<FViewportAnnotationConfig> Annotations,
		bool bShowUI = false);

	/*
	 * Gets the currently selected actors in the level editor.
	 * @return A list of the currently selected actors.
	 */
	UFUNCTION(meta = (AICallable))
	static TArray<AActor*> GetSelectedActors();

	/*
	 * Selects the specified actors in the current scene.
	 * @param Actors The actors to select.
	 */
	UFUNCTION(meta = (AICallable))
	static void SelectActors(const TArray<AActor*>& Actors);

	/*
	 * Returns the position and rotation of the level viewport camera.
	 * @return The transform for the level viewport camera.
	 */
	UFUNCTION(meta = (AICallable))
	static FTransform GetCameraTransform();

	/*
	 * Sets the position and rotation of the level viewport camera.
	 * @param Transform The transform to apply to the viewport camera.
	 */
	UFUNCTION(meta = (AICallable))
	static void SetCameraTransform(const FTransform& Transform);

	/*
	 * Repositions the level editor camera to focus on the specified actors.
	 * Cannot be called while PIE is active.
	 * @param Actors The actors to focus the level camera on.
	 */
	UFUNCTION(meta = (AICallable))
	static void FocusOnActors(const TArray<AActor*>& Actors);

	/*
	 * Returns all actors in the current level whose bounds intersect the viewport frustum.
	 * @return The actors that are at least partially in view.
	 */
	UFUNCTION(meta = (AICallable))
	static TArray<AActor*> GetVisibleActors();

	/*
	 * Converts a world-space position into normalized screen space based on the editor viewport camera.
	 * @param Position The world space position to convert.
	 * @return The normalized screen-space coordinates for the position if it is in view.
	 */
	UFUNCTION(meta = (AICallable))
	static FVector2D WorldPosToScreenCoords(FVector Position);

	/*
	 * Finds the world position of the nearest solid object at a given set of normalized view space coords.
	 * @param Coords The normalized screen-space coordinates to trace from.
	 * @param TraceDistance The maximum distance to trace within the scene.
	 * @return The world-space coordinates of the intersection point.
	 */
	UFUNCTION(meta = (AICallable))
	static FVector ScreenCoordsToWorld(FVector2D Coords, float TraceDistance = 100000.f);

	/*
	 * Gets the list of assets selected in the content browser.
	 * @return A list of package paths to the assets selected in the content browser.
	 */
	UFUNCTION(meta = (AICallable))
	static TArray<FString> GetSelectedAssets();

	/*
	 * Selects the specified assets in the content browser.
	 * Completes once the content browser has applied the selection.
	 * @param AssetPaths The package paths of the assets to select.
	 */
	UFUNCTION(meta = (AICallable))
	static UToolCallAsyncResultVoid* SelectAssets(const TArray<FString>& AssetPaths);

	/*
	 * Gets the current path of the active content browser.
	 * @return The path to the folder currently shown in the content browser.
	 */
	UFUNCTION(meta = (AICallable))
	static FString GetContentBrowserPath();

	/*
	 * Navigates the active content browser to the specified folder path.
	 * @param Path The internal path to navigate to, e.g. '/Game/Meshes'.
	 */
	UFUNCTION(meta = (AICallable))
	static void SetContentBrowserPath(const FString& Path);

	/*
	 * Opens an asset editor for the specified asset.
	 * @param AssetPath The path to the asset to open, e.g. '/Game/Meshes/SM_Cube'.
	 */
	UFUNCTION(meta = (AICallable))
	static void OpenEditorForAsset(const FString& AssetPath);

	/*
	 * Gets the list of assets currently open in asset editors.
	 * @return A list of package paths to the assets open in asset editors.
	 */
	UFUNCTION(meta = (AICallable))
	static TArray<FString> GetOpenAssets();

	/*
	 * Starts a Play-In-Editor or Simulate-In-Editor session using the current level.
	 * Completes after the engine fires PostPIEStarted (session fully started,
	 * BeginPlay called) and Options.WarmupSeconds have elapsed, giving project-
	 * specific initialization (services, authentication, plugin warmup) time to
	 * settle before the agent inspects state or logs.
	 * Raises an error if a play session is already running.
	 * @param Options Session configuration: PIE vs Simulate, play mode, optional
	 *   spawn transform override, warmup duration. See FPIESessionOptions.
	 */
	UFUNCTION(meta = (AICallable))
	static UToolCallAsyncResultVoid* StartPIE(const FPIESessionOptions& Options);

	/*
	 * Stops the currently running play session (PIE or Simulate).
	 * Raises an error if no play session is running.
	 */
	UFUNCTION(meta = (AICallable))
	static UToolCallAsyncResultVoid* StopPIE();

	/*
	 * Returns whether a Play In Editor session is currently running.
	 * @return True if PIE is active, false otherwise.
	 */
	UFUNCTION(meta = (AICallable))
	static bool IsPIERunning();

private:
	static TSharedPtr<FJsonObject> CVarToJson(IConsoleObject* CVar);

	friend class FEditorAppToolsetSpec;
};
