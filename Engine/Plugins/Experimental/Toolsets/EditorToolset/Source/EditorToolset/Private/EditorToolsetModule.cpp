// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

#include "EditorAppToolset.h"
#include "LogsToolset.h"

#define LOCTEXT_NAMESPACE "FEditorToolsetModule"

class FEditorToolsetModule : public IModuleInterface
{
	void StartupModule()
	{
		UToolsetRegistry::RegisterToolsetClass(UEditorAppToolset::StaticClass());
		UToolsetRegistry::RegisterToolsetClass(ULogsToolset::StaticClass());
	}

	void ShutdownModule()
	{
		UToolsetRegistry::UnregisterToolsetClass(ULogsToolset::StaticClass());
		UToolsetRegistry::UnregisterToolsetClass(UEditorAppToolset::StaticClass());
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FEditorToolsetModule, EditorToolset)
