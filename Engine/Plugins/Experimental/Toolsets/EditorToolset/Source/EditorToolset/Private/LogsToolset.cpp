// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogsToolset.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformOutputDevices.h"
#include "Internationalization/Regex.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/CoreMisc.h"
#include "Misc/FileHelper.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OutputDeviceNull.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LogsToolset)

namespace
{
	// Matches a standard UE log line prefix and captures the category name.
	// Example: [2024.01.15-10.30.00:000][  0]LogTemp: Warning: Some message
	// NOTE: FRegexPattern must NOT be a static variable — it requires i18n to be initialized.
	const FString CategoryPatternStr = TEXT(R"regex(^\[[\d.\-:]+\]\[\s*\d+\](\w+):)regex");

	// Valid verbosity names accepted by the log console command, derived directly
	// from ELogVerbosity so the list stays in sync if new levels are ever added.
	const TArray<FString> ValidVerbosities = []()
	{
		TArray<FString> Result;
		for (int32 i = ELogVerbosity::NoLogging; i < ELogVerbosity::NumVerbosity; ++i)
		{
			Result.Add(ToString(static_cast<ELogVerbosity::Type>(i)));
		}
		return Result;
	}();

	// Output device that captures lines written to it into an array.
	//
	// GetLogCategories, GetVerbosity, and SetVerbosity all route through
	// FSelfRegisteringExec::StaticExec with the "log" command. This is the same
	// code path used by the editor's Output Log UI and UE's own developer tooling.
	// It is the only public way to reach FLogSuppressionImplementation::ReverseAssociations,
	// the private map that owns the live FLogCategoryBase* registry. FLogSuppressionInterface
	// deliberately exposes only registration (AssociateSuppress / DisassociateSuppress),
	// not enumeration or name-based lookup.
	struct FLineCapture : public FOutputDevice
	{
		TArray<FString> Lines;

		virtual void Serialize(
			const TCHAR* V, ELogVerbosity::Type, const FName&) override
		{
			Lines.Add(FString(V));
		}
	};
}

TArray<FString> ULogsToolset::GetLogEntries(
	const FString& Category, const FString& Pattern, int32 MaxEntries)
{
	if (!Category.IsEmpty())
	{
		const TArray<FString> Known = GetLogCategories(Category);
		const bool bExists = Known.ContainsByPredicate([&Category](const FString& C)
		{
			return C.Equals(Category, ESearchCase::IgnoreCase);
		});
		if (!bExists)
		{
			UKismetSystemLibrary::RaiseScriptError(
				FString::Printf(TEXT("Log category '%s' not found."), *Category));
			return {};
		}
	}

	const FRegexPattern CategoryRegex(CategoryPatternStr);
	TOptional<FRegexPattern> PatternRegex;
	if (!Pattern.IsEmpty())
	{
		PatternRegex.Emplace(Pattern, ERegexPatternFlags::CaseInsensitive);
	}

	TArray<FString> Results;

	for (const FString& Line : ReadLogLines())
	{
		if (!Category.IsEmpty())
		{
			FRegexMatcher Matcher(CategoryRegex, Line);
			if (!Matcher.FindNext() ||
				!Matcher.GetCaptureGroup(1).Equals(Category, ESearchCase::IgnoreCase))
			{
				continue;
			}
		}

		if (PatternRegex.IsSet())
		{
			FRegexMatcher Matcher(*PatternRegex, Line);
			if (!Matcher.FindNext())
			{
				continue;
			}
		}

		Results.Add(Line);
	}

	if (MaxEntries > 0 && Results.Num() > MaxEntries)
	{
		Results.RemoveAt(0, Results.Num() - MaxEntries);
	}

	return Results;
}

TArray<FString> ULogsToolset::GetLogCategories(const FString& Filter)
{
	// "log list" asks FLogSuppressionImplementation to enumerate every FLogCategoryBase
	// that has registered itself via FLogSuppressionInterface::AssociateSuppress. Each
	// output line is formatted as "%-40s  %-12s  %s" (name, verbosity, debug-break flag).
	// Splitting on whitespace and taking the first token gives the category name.
	// "log list <filter>" narrows results to categories whose name contains the filter
	// string (case-insensitive).
	FLineCapture Capture;
	const FString Command = Filter.IsEmpty()
		? TEXT("log list")
		: FString::Printf(TEXT("log list %s"), *Filter);
	FSelfRegisteringExec::StaticExec(nullptr, *Command, Capture);

	TArray<FString> Categories;
	for (const FString& Line : Capture.Lines)
	{
		TArray<FString> Parts;
		Line.TrimEnd().ParseIntoArrayWS(Parts);
		if (Parts.Num() > 0)
		{
			Categories.Add(Parts[0]);
		}
	}

	Categories.Sort();
	return Categories;
}

FString ULogsToolset::GetVerbosity(const FString& Category)
{
	if (Category.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Category cannot be empty."));
		return FString();
	}

	// "log list <filter>" filters by substring, so it may return multiple lines.
	// Pass the exact name as the filter to keep the output small, then match exactly.
	FLineCapture Capture;
	FSelfRegisteringExec::StaticExec(
		nullptr, *FString::Printf(TEXT("log list %s"), *Category), Capture);

	for (const FString& Line : Capture.Lines)
	{
		TArray<FString> Parts;
		Line.TrimEnd().ParseIntoArrayWS(Parts);
		if (Parts.Num() >= 2 && Parts[0].Equals(Category, ESearchCase::IgnoreCase))
		{
			return Parts[1];
		}
	}

	UKismetSystemLibrary::RaiseScriptError(
		FString::Printf(TEXT("Log category '%s' not found."), *Category));
	return FString();
}

void ULogsToolset::SetVerbosity(const FString& Category, const FString& Verbosity)
{
	if (Category.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Category cannot be empty."));
		return;
	}

	const bool bValid = ValidVerbosities.ContainsByPredicate(
		[&Verbosity](const FString& V)
		{
			return V.Equals(Verbosity, ESearchCase::IgnoreCase);
		});

	if (!bValid)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("'%s' is not a valid verbosity level. Use one of: %s"),
			*Verbosity, *FString::Join(ValidVerbosities, TEXT(", "))));
		return;
	}

	FOutputDeviceNull Null;
	FSelfRegisteringExec::StaticExec(
		nullptr, *FString::Printf(TEXT("log %s %s"), *Category, *Verbosity), Null);
}

FString ULogsToolset::GetLogFilePath()
{
	const FString AbsLogPath = FPaths::ConvertRelativePathToFull(
		FPlatformOutputDevices::GetAbsoluteLogFilename());
	if (!AbsLogPath.IsEmpty() && IFileManager::Get().FileExists(*AbsLogPath))
	{
		return AbsLogPath;
	}

	UKismetSystemLibrary::RaiseScriptError(
		FString::Printf(TEXT("Log file not found: %s"), *AbsLogPath));
	return FString();
}

TArray<FString> ULogsToolset::ReadLogLines()
{
	const FString LogFilePath = GetLogFilePath();
	if (LogFilePath.IsEmpty())
	{
		return {};
	}

	FString Content;
	if (!FFileHelper::LoadFileToString(Content, *LogFilePath,
		FFileHelper::EHashOptions::None, FILEREAD_AllowWrite))
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Failed to read log file: %s"), *LogFilePath));
		return {};
	}

	TArray<FString> Lines;
	Content.ParseIntoArrayLines(Lines, /*bCullEmpty=*/false);
	Lines.RemoveAll([](const FString& Line) { return Line.TrimEnd().IsEmpty(); });
	return Lines;
}
