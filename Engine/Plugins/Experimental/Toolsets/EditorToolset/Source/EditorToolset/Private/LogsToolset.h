// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolsetRegistry/ToolsetDefinition.h"

#include "LogsToolset.generated.h"

/// Provides tools for reading the Unreal Engine output log and controlling
/// log category verbosity.
UCLASS(BlueprintType, MinimalAPI)
class ULogsToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Returns log entries from the current session's log file.
	 * @param Category If non-empty, only returns entries from this log category (e.g. "LogTemp").
	 * @param Pattern If non-empty, only returns entries whose text matches this regular
	 *   expression.
	 * @param MaxEntries Maximum number of entries to return, taken from the end of the log.
	 *   Pass 0 for no limit. Defaults to 1000.
	 * @return A list of matching log entries in chronological order.
	 */
	UFUNCTION(meta = (AICallable), Category = "LogsToolset")
	static TArray<FString> GetLogEntries(
		const FString& Category = TEXT(""), const FString& Pattern = TEXT(""), int32 MaxEntries = 1000);

	/**
	 * Returns a sorted list of registered log categories.
	 * @param Filter If non-empty, only returns categories whose name contains this
	 *   substring.
	 * @return A sorted list of log category names (e.g. ["LogBlueprint", "LogTemp"]).
	 */
	UFUNCTION(meta = (AICallable), Category = "LogsToolset")
	static TArray<FString> GetLogCategories(const FString& Filter = TEXT(""));

	/**
	 * Returns the current verbosity level for a log category.
	 * @param Category The log category name, e.g. "LogTemp".
	 * @return The verbosity level as a string: one of "NoLogging", "Fatal", "Error",
	 *   "Warning", "Display", "Log", "Verbose", or "VeryVerbose". Raises a script error
	 *   if the category is not found.
	 */
	UFUNCTION(meta = (AICallable), Category = "LogsToolset")
	static FString GetVerbosity(const FString& Category);

	/**
	 * Sets the verbosity level for a log category.
	 * @param Category The log category name, e.g. "LogTemp".
	 * @param Verbosity The verbosity level: one of "NoLogging", "Fatal", "Error",
	 *   "Warning", "Display", "Log", "Verbose", or "VeryVerbose".
	 */
	UFUNCTION(meta = (AICallable), Category = "LogsToolset")
	static void SetVerbosity(const FString& Category, const FString& Verbosity);

private:
	static FString GetLogFilePath();
	static TArray<FString> ReadLogLines();

	friend class FLogsToolsetSpec;
};
