// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "Misc/AutomationTest.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Editor.h"
#include "Engine/World.h"
#include "LevelEditorViewport.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "Misc/App.h"
#include "Misc/CoreMiscDefines.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "UObject/StrongObjectPtr.h"
#include "EditorAppToolset.h"
#include "EditorAppToolsetTestUtilities.h"
#include "ToolsetRegistry/ToolCallAsyncResultVoid.h"
#include "ToolsetRegistry/ToolCallAsyncResultImage.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"


#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	TSharedPtr<FJsonObject> ParseJsonObject(const FString& JsonString)
	{
		TSharedPtr<FJsonObject> JsonObject;
		if (JsonString.IsEmpty())
		{
			return nullptr;
		}
		const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);
		FJsonSerializer::Deserialize(Reader, JsonObject);
		return JsonObject;
	}

	template<typename CVarTypeT>
	struct FCVarTestFixture
	{
		FCVarTestFixture(const CVarTypeT& InitialValue) :
			CVarNamespace(TEXT("toolset.")),
			CVarName(FString::Printf(TEXT("%stest.TestString"), *CVarNamespace)),
			CVar(*CVarName, InitialValue, TEXT("A test cvar."))
		{
		}

		TSharedPtr<FJsonObject> FindCVarJsonData() const
		{
			TSharedPtr<FJsonObject> CVarJsonData =
				ParseJsonObject(UEditorAppToolset::SearchCVars(CVarNamespace));
			if (!CVarJsonData)
			{
				return nullptr;
			}
			const TSharedPtr<FJsonObject>* CommandJson = nullptr;
			CVarJsonData->TryGetObjectField(CVarName, CommandJson);
			return CommandJson ? *CommandJson : nullptr;
		}

		FString CVarNamespace;
		FString CVarName;
		TAutoConsoleVariable<CVarTypeT> CVar;
	};
}

BEGIN_DEFINE_SPEC(FEditorAppToolsetSpec, "AI.Toolsets.EditorToolset.EditorAppToolsetSpec",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FEditorAppToolsetSpec)

void FEditorAppToolsetSpec::Define()
{
	It(TEXT("Can find a bool cvar"), [this]()
	{
		FCVarTestFixture<bool> Fixture(true);
		TSharedPtr<FJsonObject> CommandJson = Fixture.FindCVarJsonData();
		TestTrue("CVar", CommandJson.IsValid());
		if (!CommandJson)
		{
			return;
		}
		TestEqual("CVar", CommandJson->GetStringField(
			FString(TEXT("help"))), Fixture.CVar->GetHelp());
		bool Value = false;
		TestTrue("CVar", CommandJson->TryGetBoolField(
			FString(TEXT("value")), Value));
		TestEqual("CVar", Value, Fixture.CVar->AsVariable()->GetBool());
	});

	It(TEXT("Can find an int cvar"), [this]()
	{
		FCVarTestFixture<int> Fixture(5);
		TSharedPtr<FJsonObject> CommandJson = Fixture.FindCVarJsonData();
		TestTrue("CVar", CommandJson.IsValid());
		if (!CommandJson)
		{
			return;
		}
		TestEqual("CVar", CommandJson->GetStringField(
			FString(TEXT("help"))), Fixture.CVar->GetHelp());
		int32 Value = 0;
		TestTrue("CVar", CommandJson->TryGetNumberField(FString(TEXT("value")), Value));
		TestEqual("CVar", Value, Fixture.CVar->AsVariable()->GetInt());
	});

	It(TEXT("Can find an float cvar"), [this]()
	{
		FCVarTestFixture<float> Fixture(5.0);
		TSharedPtr<FJsonObject> CommandJson = Fixture.FindCVarJsonData();
		TestTrue("CVar", CommandJson.IsValid());
		if (!CommandJson)
		{
			return;
		}
		TestEqual("CVar", CommandJson->GetStringField(
			FString(TEXT("help"))), Fixture.CVar->GetHelp());
		float Value = 0;
		TestTrue("CVar", CommandJson->TryGetNumberField(
			FString(TEXT("value")), Value));
		TestEqual("CVar", Value, Fixture.CVar->AsVariable()->GetFloat());
	});

	It(TEXT("Can find an string cvar"), [this]()
	{
		FCVarTestFixture<FString> Fixture(TEXT("testy"));
		TSharedPtr<FJsonObject> CommandJson = Fixture.FindCVarJsonData();
		TestTrue("CVar", CommandJson.IsValid());
		if (!CommandJson)
		{
			return;
		}
		TestEqual("CVar", CommandJson->GetStringField(
			FString(TEXT("help"))), Fixture.CVar->GetHelp());
		FString Value;
		TestTrue("CVar", CommandJson->TryGetStringField(
			FString(TEXT("value")), Value));
		TestEqual("CVar", Value, Fixture.CVar->AsVariable()->GetString());
	});

	It(TEXT("Can get default values from cvars"), [this]()
	{
		IConsoleManager& ConsoleManager = IConsoleManager::Get();
		FString CVarName(TEXT("toolset.test.TestBit"));
		static uint8 Force0Mask[1] = { 0 };
		static uint8 Force1Mask[1] = { 0 };
		const uint32 BitNumber = 0;
		IConsoleVariable* CVar = ConsoleManager.RegisterConsoleVariableBitRef(
			*CVarName, TEXT("MyFlag"), BitNumber, Force0Mask, Force1Mask, TEXT("Test"));
		TestTrue("CVar", !UEditorAppToolset::SearchCVars(CVarName).IsEmpty());
		ConsoleManager.UnregisterConsoleObject(CVar);
	});


	It(TEXT("Skips disabled cvar"), [this]()
	{
		FString CVarName(TEXT("toolset.test.TestDisabled"));
		TAutoConsoleVariable<bool> CVar(*CVarName, true, TEXT("A disabled cvar."), ECVF_Unregistered);
		FString CVarJsonString = UEditorAppToolset::SearchCVars(CVarName);
		TSharedPtr<FJsonObject> CVarJsonData = ParseJsonObject(CVarJsonString);
		TestTrue("CVar", CVarJsonData.IsValid());
		TestEqual("CVar", CVarJsonData->Values.Num(), 0);
	});

	// CaptureAssetImage tests

	It(TEXT("CaptureAssetImage returns an error for a non-existent asset"), [this]()
	{
		UToolCallAsyncResultImage* Result = UEditorAppToolset::CaptureAssetImage(
			TEXT("/Game/NonExistent/DoesNotExist.DoesNotExist"));
		if (TestNotNull(TEXT("Result"), Result) || !Result) return;
		// Error is set synchronously before CaptureAssetImage returns.
		TestTrue("Complete", Result->bIsComplete);
		TestFalse("Error is set", Result->Error.IsEmpty());
	});

	It(TEXT("CaptureAssetImage renders a thumbnail for a static mesh"), [this]()
	{
		if (!FApp::CanEverRender())
		{
			// Rendering is disabled; skipping thumbnail test.
			return;
		}

		UToolCallAsyncResultImage* Result = UEditorAppToolset::CaptureAssetImage(
			TEXT("/Engine/BasicShapes/Cube.Cube"));
		if (!TestNotNull("Result", Result))
		{
			return;
		}

		TStrongObjectPtr<UToolCallAsyncResultImage> StrongResult(Result);

		AddCommand(new FFunctionLatentCommand(
			[this, StrongResult]() mutable -> bool
			{
				if (!StrongResult->bIsComplete)
				{
					return false; // not done yet
				}
				TestTrue("No error", StrongResult->Error.IsEmpty());
				TestFalse("Has image data", StrongResult->Value.Data.IsEmpty());
				TestEqual("MIME type",
					StrongResult->Value.MimeType, FString(TEXT("image/png")));
				return true; // done
			}));
	});

	It(TEXT("CaptureAssetImage errors when given the current level path"), [this]()
	{
		UWorld* EditorWorld = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
		if (!EditorWorld)
		{
			AddInfo(TEXT("Skipped: no editor world available."));
			return;
		}

		const FString CurrentLevelPath = EditorWorld->GetOutermost()->GetName();
		UToolCallAsyncResultImage* Result =
			UEditorAppToolset::CaptureAssetImage(CurrentLevelPath);
		if (!TestNotNull("Result", Result))
		{
			return;
		}
		// Error is set synchronously: levels are not supported by CaptureAssetImage.
		TestTrue("Complete", Result->bIsComplete);
		TestTrue(
			FString::Printf(TEXT("Error mentions CaptureViewport: '%s'"), *Result->Error),
			Result->Error.Contains(TEXT("CaptureViewport")));
	});

	It(TEXT("CaptureEditorImage screenshots the entire editor window"), [this]()
	{
		if (GIsBuildMachine || !FApp::CanEverRender() || !GCurrentLevelEditingViewportClient)
		{
			AddInfo(TEXT("Skipped: requires an interactive level viewport (GCurrentLevelEditingViewportClient)."));
			return;
		}

		FToolsetImage Result = UEditorAppToolset::CaptureEditorImage();
		TestFalse("Has image data", Result.Data.IsEmpty());
		TestEqual("MIME type", Result.MimeType, FString(TEXT("image/png")));
	});

	It(TEXT("CaptureViewport returns a PNG with no annotations when no config is provided"), [this]()
	{
		// Reads from the current level viewport client. Skip on build machines / headless
		// runs where GCurrentLevelEditingViewportClient is not set up.
		if (GIsBuildMachine || !FApp::CanEverRender() || !GCurrentLevelEditingViewportClient)
		{
			AddInfo(TEXT("Skipped: requires an interactive level viewport (GCurrentLevelEditingViewportClient)."));
			return;
		}

		FViewportCapture Result = UEditorAppToolset::CaptureViewport({}, {});

		TestFalse("Has image data", Result.Image.Data.IsEmpty());
		TestEqual("MIME type", Result.Image.MimeType, FString(TEXT("image/png")));
		TestTrue("Camera FOV populated", Result.CameraFOV > 0.f);
		// Annotation fields stay at their defaults when no config was provided.
		TestEqual("Grid spacing default", Result.Grid.SpacingCm, 0.f);
		TestEqual("Grid extent default", Result.Grid.ExtentCm, 0.f);
		TestEqual("LabeledActors empty", Result.LabeledActors.Num(), 0);
	});

	It(TEXT("CaptureViewport echoes grid metadata when annotations are configured"), [this]()
	{
		// Reads from the current level viewport client. Skip on build machines / headless
		// runs where GCurrentLevelEditingViewportClient is not set up.
		if (GIsBuildMachine || !FApp::CanEverRender() || !GCurrentLevelEditingViewportClient)
		{
			AddInfo(TEXT("Skipped: requires an interactive level viewport (GCurrentLevelEditingViewportClient)."));
			return;
		}

		FViewportAnnotationConfig Config;
		Config.GridSpacing = 500.f;
		Config.GridExtent = 5000.f;
		Config.GridHeight = 0.f;
		Config.MaxLabelDistance = 5000.f;

		FViewportCapture Result = UEditorAppToolset::CaptureViewport({}, Config);

		TestFalse("Has image data", Result.Image.Data.IsEmpty());
		TestEqual("MIME type", Result.Image.MimeType, FString(TEXT("image/png")));
		// Grid parameters are echoed back unchanged.
		TestEqual("Grid spacing echoed", Result.Grid.SpacingCm, 500.f);
		TestEqual("Grid extent echoed", Result.Grid.ExtentCm, 5000.f);
		TestEqual("Grid Height echoed", Result.Grid.Height, 0.f);
	});

	It(TEXT("CaptureViewport with CaptureTransform reflects the requested pose and restores the viewport camera"), [this]()
	{
		// Reads from the current level viewport client. Skip on build machines / headless
		// runs where GCurrentLevelEditingViewportClient is not set up.
		if (GIsBuildMachine || !FApp::CanEverRender() || !GCurrentLevelEditingViewportClient)
		{
			AddInfo(TEXT("Skipped: requires an interactive level viewport (GCurrentLevelEditingViewportClient)."));
			return;
		}

		const FVector OriginalLocation = GCurrentLevelEditingViewportClient->GetViewLocation();
		const FRotator OriginalRotation = GCurrentLevelEditingViewportClient->GetViewRotation();

		const FVector RequestedLocation(1234.5f, -678.9f, 250.f);
		const FRotator RequestedRotation(-15.f, 45.f, 0.f);

		FViewportCapture Result = UEditorAppToolset::CaptureViewport(
			FTransform(RequestedRotation, RequestedLocation), {});

		// The captured image reports the requested pose, not the original viewport pose.
		TestFalse("Has image data", Result.Image.Data.IsEmpty());
		TestEqual("CameraLocation reflects requested pose", Result.CameraLocation, RequestedLocation);
		TestEqual("CameraRotation pitch reflects requested pose", Result.CameraRotation.Pitch, RequestedRotation.Pitch);
		TestEqual("CameraRotation yaw reflects requested pose", Result.CameraRotation.Yaw, RequestedRotation.Yaw);

		// The live viewport camera is restored on the way out.
		TestEqual("Live ViewLocation restored", GCurrentLevelEditingViewportClient->GetViewLocation(), OriginalLocation);
		const FRotator RestoredRotation = GCurrentLevelEditingViewportClient->GetViewRotation();
		TestEqual("Live ViewRotation pitch restored", RestoredRotation.Pitch, OriginalRotation.Pitch);
		TestEqual("Live ViewRotation yaw restored", RestoredRotation.Yaw, OriginalRotation.Yaw);
	});

	// Actor selection tests

	It(TEXT("GetSelectedActors and SelectActors round-trip"), [this]()
	{
		UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
		if (!World)
		{
			return;
		}

		TArray<AActor*> OriginalSelection = UEditorAppToolset::GetSelectedActors();

		AActor* TempActor = World->SpawnActor<AActor>(AActor::StaticClass());
		if (!TestNotNull("TempActor", TempActor)) return;

		ON_SCOPE_EXIT
		{
			UEditorAppToolset::SelectActors(OriginalSelection);
			TempActor->Destroy();
		};

		UEditorAppToolset::SelectActors({TempActor});
		TestTrue("TempActor is selected",
			UEditorAppToolset::GetSelectedActors().Contains(TempActor));
	});

	// Viewport camera tests

	It(TEXT("GetCameraTransform and SetCameraTransform round-trip"), [this]()
	{
		UUnrealEditorSubsystem* Subsystem = GEditor
			? GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>() : nullptr;
		if (!Subsystem)
		{
			return;
		}
		FVector OriginalLocation;
		FRotator OriginalRotation;
		if (!Subsystem->GetLevelViewportCameraInfo(OriginalLocation, OriginalRotation))
		{
			// No viewport available; skip.
			return;
		}

		const FVector TestLocation(100.f, 200.f, 300.f);
		const FRotator TestRotation(10.f, 20.f, 0.f);
		UEditorAppToolset::SetCameraTransform(FTransform(TestRotation, TestLocation));

		FTransform Result = UEditorAppToolset::GetCameraTransform();
		TestEqual("Camera location X", Result.GetLocation().X, TestLocation.X);
		TestEqual("Camera location Y", Result.GetLocation().Y, TestLocation.Y);
		TestEqual("Camera location Z", Result.GetLocation().Z, TestLocation.Z);
		TestEqual("Camera rotation pitch", Result.Rotator().Pitch, TestRotation.Pitch);
		TestEqual("Camera rotation yaw", Result.Rotator().Yaw, TestRotation.Yaw);

		// Restore original camera position.
		Subsystem->SetLevelViewportCameraInfo(OriginalLocation, OriginalRotation);
	});

	It(TEXT("FocusOnActors makes actor visible to GetVisibleActors"), [this]()
	{
		if (!GEditor || GEditor->PlayWorld)
		{
			return;
		}
		UWorld* World = GEditor->GetEditorWorldContext().World();
		if (!World)
		{
			return;
		}
		UUnrealEditorSubsystem* Subsystem =
			GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
		if (!Subsystem)
		{
			return;
		}
		FVector OriginalLocation;
		FRotator OriginalRotation;
		if (!Subsystem->GetLevelViewportCameraInfo(OriginalLocation, OriginalRotation))
		{
			// No viewport available; skip.
			return;
		}

		AActor* TempActor = World->SpawnActor<AActor>(AActor::StaticClass());
		if (!TestNotNull("TempActor", TempActor)) return;

		UEditorAppToolset::FocusOnActors({TempActor});

		AddCommand(new FUntilCommand(
			[Subsystem, TempActor, OriginalLocation, OriginalRotation]() -> bool
			{
				if (!UEditorAppToolset::GetVisibleActors().Contains(TempActor)) return false;
				Subsystem->SetLevelViewportCameraInfo(OriginalLocation, OriginalRotation);
				TempActor->Destroy();
				return true;
			},
			[this, Subsystem, TempActor, OriginalLocation, OriginalRotation]() -> bool
			{
				AddError(TEXT("Timed out waiting for actor to become visible after FocusOnActors."));
				Subsystem->SetLevelViewportCameraInfo(OriginalLocation, OriginalRotation);
				TempActor->Destroy();
				return true;
			}));
	});

	// Content browser tests

	It(TEXT("GetContentBrowserPath and SetContentBrowserPath round-trip"), [this]()
	{
		FString OriginalPath = UEditorAppToolset::GetContentBrowserPath();

		ON_SCOPE_EXIT
		{
			if (!OriginalPath.IsEmpty())
			{
				UEditorAppToolset::SetContentBrowserPath(OriginalPath);
			}
		};

		UEditorAppToolset::SetContentBrowserPath(TEXT("/Game"));
		TestEqual("Path is set", UEditorAppToolset::GetContentBrowserPath(), FString(TEXT("/Game")));
	});

	It(TEXT("SetContentBrowserPath raises a script error for an invalid path"), [this]()
	{
		UE::ToolsetRegistry::FToolCallExceptionHandler Handler;
		Handler.CaptureErrorsIn([]
		{
			UEditorAppToolset::SetContentBrowserPath(TEXT("/InvalidPath/DoesNotExist"));
		});
		TestTrue("Script error raised", Handler.GetException().Contains(TEXT("Failed to navigate")));
	});

	It(TEXT("GetSelectedAssets and SelectAssets round-trip"), [this]()
	{
		if (FApp::IsUnattended())
		{
			AddInfo(TEXT("Skipping: content browser selection requires an active editor UI."));
			return;
		}

		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		AssetRegistry.WaitForCompletion(); // ensure background scan is done before lookup
		TArray<FAssetData> GameAssets;
		AssetRegistry.GetAssetsByPath(
			TEXT("/Game"), GameAssets, /*bRecursive=*/true);
		if (GameAssets.IsEmpty())
		{
			// No game assets available to select; skip.
			return;
		}
		const FString TestAsset = GameAssets[0].PackageName.ToString();
		TArray<FString> OriginalSelection = UEditorAppToolset::GetSelectedAssets();

		UToolCallAsyncResultVoid* Result = UEditorAppToolset::SelectAssets({TestAsset});
		if (!TestNotNull(TEXT("Result"), Result)) return;

		TStrongObjectPtr<UToolCallAsyncResultVoid> StrongResult(Result);
		AddCommand(new FFunctionLatentCommand(
			[this, StrongResult, TestAsset, OriginalSelection]() mutable -> bool
			{
				if (!StrongResult->bIsComplete) return false;
				TestEqual(TEXT("No error"), StrongResult->Error, TEXT(""));
				TArray<FString> SelectedAssets = UEditorAppToolset::GetSelectedAssets();
				TestTrue(
					FString::Printf(
						TEXT("Asset '%s' is selected in [%s]"),
						*TestAsset,
						*FString::Join(SelectedAssets, TEXT(", "))),
					SelectedAssets.Contains(TestAsset));
				UEditorAppToolset::SelectAssets(OriginalSelection);
				StrongResult.Reset();
				return true;
			}));
	});

	It(TEXT("GetOpenAssets returns assets open in asset editors"), [this]()
	{
		if (FApp::IsUnattended())
		{
			AddInfo(TEXT("Skipping: opening asset editors requires an active editor UI."));
			return;
		}

		// Open a known engine asset, verify it appears in GetOpenAssets, then close it.
		const FString AssetPath = TEXT("/Engine/BasicShapes/Cube");
		UObject* Asset = StaticLoadObject(UObject::StaticClass(), nullptr, TEXT("/Engine/BasicShapes/Cube.Cube"));
		if (!TestNotNull("Cube asset", Asset)) return;

		UAssetEditorSubsystem* Subsystem = GEditor
			? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>() : nullptr;
		if (!TestNotNull("AssetEditorSubsystem", Subsystem)) return;

		Subsystem->OpenEditorForAsset(Asset);

		ON_SCOPE_EXIT { Subsystem->CloseAllEditorsForAsset(Asset); };

		TestTrue("Cube is listed",
			UEditorAppToolset::GetOpenAssets().Contains(AssetPath));
	});

	It(TEXT("OpenEditorForAsset raises a script error for a non-existent asset"), [this]()
	{
		if (FApp::IsUnattended())
		{
			AddInfo(TEXT("Skipping: opening asset editors requires an active editor UI."));
			return;
		}
		UE::ToolsetRegistry::FToolCallExceptionHandler Handler;
		Handler.CaptureErrorsIn([]
		{
			UEditorAppToolset::OpenEditorForAsset(TEXT("/Game/NonExistent/DoesNotExist"));
		});
		TestTrue("Script error raised", Handler.GetException().Contains(TEXT("is not a valid asset path")));
	});

	// PIE tests

	It(TEXT("StartPIE and StopPIE round-trip, IsPIERunning reflects state"), [this]()
	{
		FPIESessionOptions Options;
		Options.WarmupSeconds = 0.1f;
		UE::EditorToolset::Testing::RunPlaySessionTest(*this,
			[Options]() { return UEditorAppToolset::StartPIE(Options); },
			[]() { /* helper already asserts no error + IsPIERunning */ });
	});

	It(TEXT("StopPIE errors synchronously when no session is running"), [this]()
	{
		if (UEditorAppToolset::IsPIERunning())
		{
			AddInfo(TEXT("Skipped: PIE is already running from a prior test."));
			return;
		}

		UToolCallAsyncResultVoid* Result = UEditorAppToolset::StopPIE();
		if (!TestNotNull("Result", Result))
		{
			return;
		}
		TestTrue("Complete synchronously", Result->bIsComplete);
		TestTrue(FString::Printf(TEXT("Error mentions 'not currently running': '%s'"), *Result->Error),
		Result->Error.Contains(TEXT("not currently running")));
	});

	It(TEXT("StartPIE in Simulate mode reflects IsSimulatingInEditor"), [this]()
	{
		FPIESessionOptions Options;
		Options.bSimulate = true;
		Options.WarmupSeconds = 0.1f;
		UE::EditorToolset::Testing::RunPlaySessionTest(*this,
			[Options]() { return UEditorAppToolset::StartPIE(Options); },
			[this]()
			{
				TestTrue("GEditor reports Simulate mode", GEditor && GEditor->IsSimulatingInEditor());
			});
	});

	It(TEXT("StartPIE with StartTransform spawns the player pawn at the requested transform"), [this]()
	{
		const FVector ExpectedLocation(1234.5f, 6789.0f, 500.0f);
		const FRotator ExpectedRotation(0.f, 90.f, 0.f);
		const FTransform ExpectedTransform(ExpectedRotation, ExpectedLocation);

		FPIESessionOptions Options;
		Options.StartTransform = ExpectedTransform;
		Options.WarmupSeconds = 0.1f;

		UE::EditorToolset::Testing::RunPlaySessionTest(*this,
			[Options]() { return UEditorAppToolset::StartPIE(Options); },
			[this, ExpectedLocation]()
			{
				// Verify the pawn spawned at (or very near) the requested location.
				// XY drift shouldn't happen; Z may drift within ~0.1s of gravity.
				// Tolerance is deliberately generous to catch "transform ignored entirely"
				// without being fragile to physics settling.
				// Guard chain instead of chained ternaries to avoid PVS V623 (GEditor->PlayWorld
				// is TObjectPtr<UWorld>; ?: would construct/destruct a TObjectPtr temporary).
				APawn* Pawn = nullptr;
				if (GEditor && GEditor->PlayWorld)
				{
					if (APlayerController* PC = GEditor->PlayWorld->GetFirstPlayerController())
					{
						Pawn = PC->GetPawn();
					}
				}
					
				if (Pawn == nullptr)
				{
					AddInfo(TEXT("No player pawn spawned in this level; skipping location check."));
					return;
				}

				const FVector Actual = Pawn->GetActorLocation();
				const float HorizontalDist = FVector::Dist2D(Actual, ExpectedLocation);
				const float VerticalDist = FMath::Abs(Actual.Z - ExpectedLocation.Z);
				TestTrue(FString::Printf(TEXT("Pawn XY within 10cm of requested (actual=%s, expected=%s)"),
						*Actual.ToString(), *ExpectedLocation.ToString()),
					HorizontalDist < 10.f);
				TestTrue(TEXT("Pawn Z within 200cm of requested (gravity tolerance)"),VerticalDist < 200.f);
			});
	});
}

#endif
