// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorAppToolsetTestUtilities.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "EditorAppToolset.h"
#include "Misc/App.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"
#include "Tests/AutomationEditorCommon.h"
#include "ToolsetRegistry/JsonValueOrError.h"
#include "ToolsetRegistry/ToolCallAsyncResultFutureHandler.h"
#include "ToolsetRegistry/ToolCallAsyncResultVoid.h"

namespace UE::EditorToolset::Testing
{
	namespace
	{
		// Engine-shipped empty template map. Used so PIE tests don't pick up
		// project-specific Blueprint runtime errors (e.g. AI services with null
		// references) from whichever map the dev happened to have open in the editor.
		// Tests will leave this map loaded — devs running locally re-open their work
		// map manually after running the suite.
		constexpr const TCHAR* TestMapPath = TEXT("/Engine/Maps/Templates/Template_Default");

		// The PIE watchers (FPIEStartupWatcher / FPIEShutdownWatcher) own the only real
		// deadline — they SetError on the result after 30 s. So this latent command just
		// polls the future and exits when it resolves; no second timeout is needed. The
		// framework's outer per-test timeout is the final safety net.
		void AwaitPlaySessionResult(
			FAutomationTestBase& Test,
			UToolCallAsyncResultVoid* Result,
			const TCHAR* ErrorContext,
			TFunction<void()>&& OnSuccess)
		{
			using UE::ToolsetRegistry::FJsonValueOrError;
			TSharedRef<TFuture<FJsonValueOrError>> SharedFuture =
				MakeShared<TFuture<FJsonValueOrError>>(
					UToolCallAsyncResultFutureHandler::Create(Result)->GetValueAsJson());

			Test.AddCommand(new FFunctionLatentCommand(
				[&Test, SharedFuture,
				 ErrorContext = FString(ErrorContext),
				 OnSuccess = MoveTemp(OnSuccess)]() mutable -> bool
				{
					if (!SharedFuture->IsReady())
					{
						return false;
					}
					FJsonValueOrError JsonResult = SharedFuture->Consume();
					if (JsonResult.HasError())
					{
						Test.AddError(FString::Printf(
							TEXT("%s error: %s"), *ErrorContext, *JsonResult.GetError()));
						return true;
					}
					OnSuccess();
					return true;
				}));
		}
	}

	void RunPlaySessionTest(FAutomationTestBase& Test, TFunction<UToolCallAsyncResultVoid*()> StartSession, TFunction<void()> Verify)
	{
		if (!FApp::CanEverRender() || GIsBuildMachine)
		{
			Test.AddInfo(TEXT("Skipped: no rendering available on this run."));
			return;
		}
		if (UEditorAppToolset::IsPIERunning())
		{
			Test.AddInfo(TEXT("Skipped: PIE is already running from a prior test."));
			return;
		}

		// Load the engine template map so we run PIE against a known-clean world
		// instead of whatever the dev has open. Mirrors the FEditorLoadMap pattern
		// used by EditorTests / EditorAutomationTests.
		FAutomationEditorCommonUtils::LoadMap(TestMapPath);

		UToolCallAsyncResultVoid* StartResult = StartSession();
		if (!Test.TestNotNull("StartResult", StartResult))
		{
			return;
		}

		AwaitPlaySessionResult(Test, StartResult, TEXT("StartPIE"),
			[&Test, Verify = MoveTemp(Verify)]
			{
				Test.TestTrue("Session is running after Start", UEditorAppToolset::IsPIERunning());
				Verify();
			});

		Test.AddCommand(new FFunctionLatentCommand(
			[&Test]() mutable -> bool
			{
				UToolCallAsyncResultVoid* StopResult = UEditorAppToolset::StopPIE();
				if (!Test.TestNotNull("StopResult", StopResult))
				{
					return true;
				}
				AwaitPlaySessionResult(Test, StopResult, TEXT("StopPIE"),
					[&Test]
					{
						Test.TestFalse("Session not running after Stop", UEditorAppToolset::IsPIERunning());
					});
				return true;
			}));
	}
}

#endif // WITH_DEV_AUTOMATION_TESTS
