// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "CoreMinimal.h"
#include "Templates/Function.h"

class FAutomationTestBase;
class UToolCallAsyncResultVoid;

namespace UE::EditorToolset::Testing
{
	/** Drives a full play-session round-trip test against EditorAppToolset.
	 1. Skips via AddInfo when the environment can't support PIE
	    (!FApp::CanEverRender() || GIsBuildMachine, or PIE already running).
	 2. Loads /Engine/Maps/Templates/Template_Default so PIE runs against a
	    project-agnostic clean world (avoids picking up runtime BP errors from
	    whatever the dev had open).
	 3. Calls StartSession (expected to return an in-flight UToolCallAsyncResultVoid*
	    from StartPIE) and waits on the result via UToolCallAsyncResultFutureHandler;
	    on success asserts IsPIERunning() and calls Verify for test-specific checks.
	 4. Schedules a follow-up FFunctionLatentCommand that calls StopPIE and waits
	    for it to resolve, asserting no error and IsPIERunning() is false.
	Errors at either start or stop are reported via AddError with a descriptive
	prefix. The PIE watchers' internal 30 s timeouts are the only deadlines; the
	framework's outer per-test timeout is the final safety net if a watcher
	itself fails to fire. */
	void RunPlaySessionTest(FAutomationTestBase& Test, TFunction<UToolCallAsyncResultVoid*()> StartSession, TFunction<void()> Verify);
}

#endif // WITH_DEV_AUTOMATION_TESTS
