// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "Misc/CString.h"
#include "Misc/Guid.h"

#include "LogsToolset.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"

#if WITH_DEV_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(LogLogsToolsetTest, Log, All);

BEGIN_DEFINE_SPEC(FLogsToolsetSpec, "AI.Toolsets.EditorToolset.LogsToolset",
	EAutomationTestFlags::EditorContext
	| EAutomationTestFlags::ProductFilter
	| EAutomationTestFlags::CriticalPriority)

	bool TestContains(
		const FString& Context, const FString& ToSearch, const FString& ToFind,
		ESearchCase::Type Case = ESearchCase::CaseSensitive)
	{
		return TestTrue(
			FString::Printf(TEXT("%s: `%s` in `%s`"), *Context, *ToFind, *ToSearch),
			ToSearch.Contains(ToFind, Case));
	}
END_DEFINE_SPEC(FLogsToolsetSpec)

#define TestLogCategory LogLogsToolsetTest

void FLogsToolsetSpec::Define()
{
	using namespace UE::ToolsetRegistry;

	Describe(TEXT("GetLogCategories"), [this]()
	{
		It(TEXT("Returns a non-empty sorted list"), [this]()
		{
			UE_LOGF(LogLogsToolsetTest, Log, "GetLogCategories probe");
			GLog->Flush();

			TArray<FString> Categories = ULogsToolset::GetLogCategories();
			TestGreaterThan(TEXT("Categories is non-empty"), Categories.Num(), 0);

			TArray<FString> Sorted = Categories;
			Sorted.Sort();
			TestEqual(TEXT("Categories is sorted"), Categories, Sorted);
		});

		It(TEXT("Includes the test log category"), [this]()
		{
			UE_LOGF(LogLogsToolsetTest, Log, "Category presence probe");
			GLog->Flush();

			TArray<FString> Categories = ULogsToolset::GetLogCategories();
			TestTrue(
				FString::Printf(
					TEXT("LogLogsToolsetTest is present in [%s]"),
					*FString::Join(Categories, TEXT(", "))),
				Categories.Contains(TEXT("LogLogsToolsetTest")));
		});

		It(TEXT("Filter narrows results to matching categories"), [this]()
		{
			UE_LOGF(LogLogsToolsetTest, Log, "Filter probe");
			GLog->Flush();

			TArray<FString> Filtered = ULogsToolset::GetLogCategories(TEXT("LogLogsToolsetTest"));
			TestGreaterThan(TEXT("At least one result"), Filtered.Num(), 0);
			for (const FString& Category : Filtered)
			{
				TestContains(
					TEXT("Category contains filter string"),
					Category, TEXT("LogLogsToolsetTest"), ESearchCase::IgnoreCase);
			}
		});
	});

	Describe(TEXT("GetLogEntries"), [this]()
	{
		It(TEXT("Returns a non-empty list with no filters"), [this]()
		{
			TArray<FString> Entries = ULogsToolset::GetLogEntries(TEXT(""), TEXT(""), 0);
			TestGreaterThan(TEXT("Entries is non-empty"), Entries.Num(), 0);
		});

		It(TEXT("Filters by category"), [this]()
		{
			UE_LOGF(LogLogsToolsetTest, Log, "Category filter probe");
			GLog->Flush();

			TArray<FString> Entries = ULogsToolset::GetLogEntries(
				TEXT("LogLogsToolsetTest"), TEXT(""), 0);
			TestGreaterThan(TEXT("At least one entry"), Entries.Num(), 0);
			for (const FString& Entry : Entries)
			{
				TestContains(
					TEXT("Entry contains category prefix"), Entry, TEXT("]LogLogsToolsetTest:"));
			}
		});

		It(TEXT("Filters by pattern"), [this]()
		{
			const FString Marker = FString::Printf(
				TEXT("LogsToolsetPatternMarker_%s"), *FGuid::NewGuid().ToString());
			UE_LOGF(LogLogsToolsetTest, Log, "%ls", *Marker);
			GLog->Flush();

			TArray<FString> Entries = ULogsToolset::GetLogEntries(TEXT(""), *Marker, 0);
			if (TestEqual(TEXT("Exactly one entry matches"), Entries.Num(), 1))
			{
				TestContains(TEXT("Entry contains marker"), Entries[0], Marker);
			}
		});

		It(TEXT("Pattern matching is case-insensitive"), [this]()
		{
			const FString Marker = FString::Printf(
				TEXT("LogsToolsetCaseMarker_%s"), *FGuid::NewGuid().ToString());
			UE_LOGF(LogLogsToolsetTest, Log, "%ls", *Marker);
			GLog->Flush();

			TArray<FString> Entries = ULogsToolset::GetLogEntries(
				TEXT(""), *Marker.ToLower(), 0);
			TestEqual(TEXT("Case-insensitive match finds entry"), Entries.Num(), 1);
		});

		It(TEXT("Raises an error for an unknown category"), [this]()
		{
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([this]() -> void
				{
					ULogsToolset::GetLogEntries(
						TEXT("LogThisCategoryDoesNotExist_XYZ123"),
						TEXT(""),
						0);
				});
			TestContains(
				TEXT("GetLogEntries with invalid category raises error"),
				*ExceptionHandler.GetException(), TEXT("not found"));
		});

		It(TEXT("Respects MaxEntries limit"), [this]()
		{
			constexpr int32 Limit = 5;
			for (int32 i = 0; i < Limit * 2; ++i)
			{
				UE_LOGF(TestLogCategory, Verbose, "Log testing %d", i);
			}
			FString LogCategory = TestLogCategory.GetCategoryName().ToString();
			TArray<FString> All = ULogsToolset::GetLogEntries(LogCategory, TEXT(""), 0);
			TArray<FString> Limited = ULogsToolset::GetLogEntries(LogCategory, TEXT(""), Limit);
			TestLessEqual(TEXT("At most MaxEntries returned"), Limited.Num(), Limit);
			if (All.Num() >= Limit)
			{
				TestEqual(TEXT("Returns the most recent entries"),
					Limited, TArray<FString>(All.GetData() + All.Num() - Limit, Limit));
			}
		});

		It(TEXT("Combines category and pattern filters"), [this]()
		{
			const FString Marker = FString::Printf(
				TEXT("LogsToolsetCombinedMarker_%s"), *FGuid::NewGuid().ToString());
			UE_LOGF(LogLogsToolsetTest, Log, "%ls", *Marker);
			GLog->Flush();

			TArray<FString> Entries = ULogsToolset::GetLogEntries(
				TEXT("LogLogsToolsetTest"), *Marker, 0);
			TestEqual(TEXT("Exactly one combined match"), Entries.Num(), 1);
		});
	});

	Describe(TEXT("GetVerbosity"), [this]()
	{
		It(TEXT("Returns a verbosity string for a known category"), [this]()
		{
			UE_LOGF(LogLogsToolsetTest, Log, "GetVerbosity probe");
			const FString Verbosity = ULogsToolset::GetVerbosity(TEXT("LogLogsToolsetTest"));
			TestNotEqual(TEXT("Verbosity is non-empty"), *Verbosity, TEXT(""));
		});

		It(TEXT("Returns empty string for an unknown category"), [this]()
		{
			const FString Verbosity =
				ULogsToolset::GetVerbosity(TEXT("LogThisCategoryDoesNotExist_XYZ123"));
			TestEqual(TEXT("Returns empty string for unknown category"), Verbosity, TEXT(""));
		});

		It(TEXT("Raises an error for an unknown category"), [this]()
		{
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([this]() -> void
				{
					ULogsToolset::GetVerbosity(TEXT("LogThisCategoryDoesNotExist_XYZ123"));
				});
			TestContains(
				TEXT("GetVerbosity raises error with an unknown category"),
				ExceptionHandler.GetException(), TEXT("not found"));
		});

		It(TEXT("Raises an error for an empty category"), [this]()
		{
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([this]() -> void
				{
					ULogsToolset::GetVerbosity(TEXT(""));
				});
			TestContains(
				TEXT("GetVerbosity raises error with an empty category"),
				ExceptionHandler.GetException(), TEXT("Category cannot be empty"));
		});
	});

	Describe(TEXT("SetVerbosity"), [this]()
	{
		It(TEXT("Can round-trip a verbosity change"), [this]()
		{
			const FString Category = TEXT("LogLogsToolsetTest");
			const FString Original = ULogsToolset::GetVerbosity(Category);
			if (!TestNotEqual(TEXT("Category verbosity is readable"), *Original, TEXT("")))
			{
				return;
			}

			ULogsToolset::SetVerbosity(Category, TEXT("Warning"));
			TestEqual(TEXT("Verbosity set to Warning"),
				ULogsToolset::GetVerbosity(Category), FString(TEXT("Warning")));

			ULogsToolset::SetVerbosity(Category, Original);
			TestEqual(TEXT("Verbosity restored"),
				ULogsToolset::GetVerbosity(Category), Original);
		});

		It(TEXT("Raises an error for an empty category"), [this]()
		{
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([this]() -> void
				{
					ULogsToolset::SetVerbosity(TEXT(""), TEXT("Display"));
				});
			TestContains(
				TEXT("SetVerbosity raises error with an empty category"),
				ExceptionHandler.GetException(), TEXT("Category cannot be empty"));
		});

		It(TEXT("Raises an error for an invalid verbosity"), [this]()
		{
			FToolCallExceptionHandler ExceptionHandler;
			ExceptionHandler.CaptureErrorsIn([this]() -> void
				{
					ULogsToolset::SetVerbosity(
						TestLogCategory.GetCategoryName().ToString(),
						TEXT("NotAVerbosity"));
				});
			TestContains(
				TEXT("SetVerbosity raises error with an invalid category"),
				ExceptionHandler.GetException(), TEXT("is not a valid verbosity level"));
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
